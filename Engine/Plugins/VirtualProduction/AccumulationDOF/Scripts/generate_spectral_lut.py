'''
Generates a spectral weight LUT for chromatic aberration simulation.

Converts wavelengths (380-700nm) to normalized linear sRGB using:

* CIE 1931 2-degree observer XYZ values (Wyman et al. multi-lobe Gaussian fit from 2013)
* XYZ to linear sRGB conversion
* Normalize such that R+G+B = 1 (pure hue), or return zeros if sum is negligible

Reference (found through Eric's hackathon project in 2024):
  "Simple Analytic Approximations to the CIE XYZ Color Matching Functions" by Wyman, Sloan, Shirley.
  Journal of Computer Graphics Techniques, 2013.
  https://jcgt.org/published/0002/02/01/

'''

import math


def xFit_1931(wave: float) -> float:
    t1 = (wave - 442.0) * (0.0624 if wave < 442.0 else 0.0374)
    t2 = (wave - 599.8) * (0.0264 if wave < 599.8 else 0.0323)
    t3 = (wave - 501.1) * (0.0490 if wave < 501.1 else 0.0382)
    return 0.362 * math.exp(-0.5 * t1 * t1) + 1.056 * math.exp(-0.5 * t2 * t2) - 0.065 * math.exp(-0.5 * t3 * t3)


def yFit_1931(wave: float) -> float:
    t1 = (wave - 568.8) * (0.0213 if wave < 568.8 else 0.0247)
    t2 = (wave - 530.9) * (0.0613 if wave < 530.9 else 0.0322)
    return 0.821 * math.exp(-0.5 * t1 * t1) + 0.286 * math.exp(-0.5 * t2 * t2)


def zFit_1931(wave: float) -> float:
    t1 = (wave - 437.0) * (0.0845 if wave < 437.0 else 0.0278)
    t2 = (wave - 459.0) * (0.0385 if wave < 459.0 else 0.0725)
    return 1.217 * math.exp(-0.5 * t1 * t1) + 0.681 * math.exp(-0.5 * t2 * t2)


def xyz_to_linear_srgb(X: float, Y: float, Z: float) -> tuple[float, float, float]:
    R =  3.2404542 * X - 1.5371385 * Y - 0.4985314 * Z
    G = -0.9692660 * X + 1.8760108 * Y + 0.0415560 * Z
    B =  0.0556434 * X - 0.2040259 * Y + 1.0572252 * Z
    return R, G, B


def wavelength_to_normalized_srgb(wavelength_nm: float) -> tuple[float, float, float]:
    
    # Get CIE XYZ
    X = xFit_1931(wavelength_nm)
    Y = yFit_1931(wavelength_nm)
    Z = zFit_1931(wavelength_nm)

    # Convert to linear sRGB
    R, G, B = xyz_to_linear_srgb(X, Y, Z)

    # Clamp negative values
    R = max(0.0, R)
    G = max(0.0, G)
    B = max(0.0, B)

    # Normalize such that R+G+B = 1
    total = R + G + B
    if total > 1e-10:
        R /= total
        G /= total
        B /= total

    return R, G, B


def generate_lut(num_entries: int = 32, wavelength_min: float = 380.0, wavelength_max: float = 700.0):

    lut = []

    for i in range(num_entries):
        t = i / (num_entries - 1) if num_entries > 1 else 0.5
        wavelength = wavelength_min + t * (wavelength_max - wavelength_min)
        R, G, B = wavelength_to_normalized_srgb(wavelength)
        lut.append((wavelength, R, G, B))

    return lut


def print_cpp_lut(lut):
    ''' Print LUT as C++ static constexpr array
    '''

    print("// C++")
    print("static constexpr float SpectralLUT[32][3] = {")

    for wavelength, R, G, B in lut:
        print(f"\t{{ {R:.6f}f, {G:.6f}f, {B:.6f}f }}, // {wavelength:.1f} nm")

    print("};")


def print_hlsl_lut(lut):
    ''' Print LUT as HLSL static const array
    '''

    print("// HLSL")
    print("static const float3 SpectralLUT[32] = {")
    
    for wavelength, R, G, B in lut:
        print(f"\tfloat3({R:.6f}, {G:.6f}, {B:.6f}), // {wavelength:.1f} nm")

    print("};")


if __name__ == "__main__":
    
    lut = generate_lut(32, 380.0, 700.0)

    print("Spectral Weight LUT Generator")
    print("CIE 1931 2-deg -> XYZ -> linear sRGB -> normalize")
    print()

    print_cpp_lut(lut)
    print()
    print_hlsl_lut(lut)
