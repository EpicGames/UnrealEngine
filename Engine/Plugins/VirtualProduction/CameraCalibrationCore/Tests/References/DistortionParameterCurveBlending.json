{
	"suiteName": "DistortionParameterCurveBlending",
	"calibrationData": [
		{
			"focus": 0,
			"zoom": 0,
			"distortion":
			{
				"parameters": [ 1, -1, 0, 0, 0 ]
			},
			"imageCenter":
			{
				"principalPoint":
				{
					"x": 8,
					"y": 7
				}
			},
			"focalLength":
			{
				"fxFy":
				{
					"x": 3,
					"y": 2
				}
			}
		},
		{
			"focus": 0,
			"zoom": 1,
			"distortion":
			{
				"parameters": [ 2, 0, 0, 0, 1 ]
			},
			"imageCenter":
			{
				"principalPoint":
				{
					"x": 9,
					"y": 8
				}
			},
			"focalLength":
			{
				"fxFy":
				{
					"x": 5,
					"y": 3
				}
			}
		},
		{
			"focus": 1,
			"zoom": 0,
			"distortion":
			{
				"parameters": [ 0, 1, 0, 0, -2 ]
			},
			"imageCenter":
			{
				"principalPoint":
				{
					"x": 2,
					"y": 4
				}
			},
			"focalLength":
			{
				"fxFy":
				{
					"x": 4,
					"y": 6
				}
			}
		},
		{
			"focus": 1,
			"zoom": 1,
			"distortion":
			{
				"parameters": [ 1, 2, 0, 0, 0 ]
			},
			"imageCenter":
			{
				"principalPoint":
				{
					"x": 4,
					"y": 7
				}
			},
			"focalLength":
			{
				"fxFy":
				{
					"x": 9,
					"y": 8
				}
			}
		}
	],
	"testCases": [
		{
			"name": "ExactCorner_Distortion_Focus0_Zoom0",
			"focus": 0,
			"zoom": 0,
			"dataType": "Distortion",
			"evaluationMethod": "LensFile",
			"expectedResult": [ 1, -1, 0, 0, 0 ]
		},
		{
			"name": "ExactCorner_ImageCenter_Focus0_Zoom0",
			"focus": 0,
			"zoom": 0,
			"dataType": "ImageCenter",
			"evaluationMethod": "LensFile",
			"expectedResult": [ 8, 7 ]
		},
		{
			"name": "ExactCorner_FocalLength_Focus0_Zoom0",
			"focus": 0,
			"zoom": 0,
			"dataType": "FocalLength",
			"evaluationMethod": "LensFile",
			"expectedResult": [ 3, 2 ]
		},
		{
			"name": "ExactCorner_HandlerDistortion_Focus0_Zoom0",
			"focus": 0,
			"zoom": 0,
			"dataType": "Distortion",
			"evaluationMethod": "Handler",
			"expectedResult": [ 1, -1, 0, 0, 0 ]
		},
		{
			"name": "ExactCorner_Distortion_Focus0_Zoom10",
			"focus": 0,
			"zoom": 1,
			"dataType": "Distortion",
			"evaluationMethod": "LensFile",
			"expectedResult": [ 2, 0, 0, 0, 1 ]
		},
		{
			"name": "ExactCorner_ImageCenter_Focus0_Zoom10",
			"focus": 0,
			"zoom": 1,
			"dataType": "ImageCenter",
			"evaluationMethod": "LensFile",
			"expectedResult": [ 9, 8 ]
		},
		{
			"name": "ExactCorner_FocalLength_Focus0_Zoom10",
			"focus": 0,
			"zoom": 1,
			"dataType": "FocalLength",
			"evaluationMethod": "LensFile",
			"expectedResult": [ 5, 3 ]
		},
		{
			"name": "ExactCorner_HandlerDistortion_Focus0_Zoom10",
			"focus": 0,
			"zoom": 1,
			"dataType": "Distortion",
			"evaluationMethod": "Handler",
			"expectedResult": [ 2, 0, 0, 0, 1 ]
		},
		{
			"name": "ExactCorner_Distortion_Focus10_Zoom0",
			"focus": 1,
			"zoom": 0,
			"dataType": "Distortion",
			"evaluationMethod": "LensFile",
			"expectedResult": [ 0, 1, 0, 0, -2 ]
		},
		{
			"name": "ExactCorner_ImageCenter_Focus10_Zoom0",
			"focus": 1,
			"zoom": 0,
			"dataType": "ImageCenter",
			"evaluationMethod": "LensFile",
			"expectedResult": [ 2, 4 ]
		},
		{
			"name": "ExactCorner_FocalLength_Focus10_Zoom0",
			"focus": 1,
			"zoom": 0,
			"dataType": "FocalLength",
			"evaluationMethod": "LensFile",
			"expectedResult": [ 4, 6 ]
		},
		{
			"name": "ExactCorner_HandlerDistortion_Focus10_Zoom0",
			"focus": 1,
			"zoom": 0,
			"dataType": "Distortion",
			"evaluationMethod": "Handler",
			"expectedResult": [ 0, 1, 0, 0, -2 ]
		},
		{
			"name": "ExactCorner_Distortion_Focus10_Zoom10",
			"focus": 1,
			"zoom": 1,
			"dataType": "Distortion",
			"evaluationMethod": "LensFile",
			"expectedResult": [ 1, 2, 0, 0, 0 ]
		},
		{
			"name": "ExactCorner_ImageCenter_Focus10_Zoom10",
			"focus": 1,
			"zoom": 1,
			"dataType": "ImageCenter",
			"evaluationMethod": "LensFile",
			"expectedResult": [ 4, 7 ]
		},
		{
			"name": "ExactCorner_FocalLength_Focus10_Zoom10",
			"focus": 1,
			"zoom": 1,
			"dataType": "FocalLength",
			"evaluationMethod": "LensFile",
			"expectedResult": [ 9, 8 ]
		},
		{
			"name": "ExactCorner_HandlerDistortion_Focus10_Zoom10",
			"focus": 1,
			"zoom": 1,
			"dataType": "Distortion",
			"evaluationMethod": "Handler",
			"expectedResult": [ 1, 2, 0, 0, 0 ]
		},
		{
			"name": "SingleCurve_Distortion_Focus0_Zoom025",
			"focus": 0,
			"zoom": 0.25,
			"dataType": "Distortion",
			"evaluationMethod": "LensFile",
			"expectedResult": [ 1.15625, -0.84375, 0, 0, 0.15625 ]
		},
		{
			"name": "SingleCurve_ImageCenter_Focus0_Zoom025",
			"focus": 0,
			"zoom": 0.25,
			"dataType": "ImageCenter",
			"evaluationMethod": "LensFile",
			"expectedResult": [ 8.15625, 7.15625 ]
		},
		{
			"name": "SingleCurve_Distortion_Focus0_Zoom075",
			"focus": 0,
			"zoom": 0.75,
			"dataType": "Distortion",
			"evaluationMethod": "LensFile",
			"expectedResult": [ 1.84375, -0.15625, 0, 0, 0.84375 ]
		},
		{
			"name": "SingleCurve_ImageCenter_Focus0_Zoom075",
			"focus": 0,
			"zoom": 0.75,
			"dataType": "ImageCenter",
			"evaluationMethod": "LensFile",
			"expectedResult": [ 8.84375, 7.84375 ]
		},
		{
			"name": "SingleCurve_FocalLength_Focus0_Zoom075",
			"focus": 0,
			"zoom": 0.75,
			"dataType": "FocalLength",
			"evaluationMethod": "LensFile",
			"expectedResult": [ 4.6875, 2.84375 ]
		},
		{
			"name": "SingleCurve_Distortion_Focus1_Zoom025",
			"focus": 1,
			"zoom": 0.25,
			"dataType": "Distortion",
			"evaluationMethod": "LensFile",
			"expectedResult": [ 0.15625, 1.15625, 0, 0, -1.6875 ]
		},
		{
			"name": "SingleCurve_ImageCenter_Focus1_Zoom025",
			"focus": 1,
			"zoom": 0.25,
			"dataType": "ImageCenter",
			"evaluationMethod": "LensFile",
			"expectedResult": [ 2.3125, 4.46875 ]
		},
		{
			"name": "SingleCurve_FocalLength_Focus1_Zoom025",
			"focus": 1,
			"zoom": 0.25,
			"dataType": "FocalLength",
			"evaluationMethod": "LensFile",
			"expectedResult": [ 4.78125, 6.3125 ]
		},
		{
			"name": "SingleCurve_Distortion_Focus025_Zoom0",
			"focus": 0.25,
			"zoom": 0,
			"dataType": "Distortion",
			"evaluationMethod": "LensFile",
			"expectedResult": [ 0.75, -0.5, 0, 0, -0.5 ]
		},
		{
			"name": "SingleCurve_ImageCenter_Focus025_Zoom0",
			"focus": 0.25,
			"zoom": 0,
			"dataType": "ImageCenter",
			"evaluationMethod": "LensFile",
			"expectedResult": [ 6.5, 6.25 ]
		},
		{
			"name": "SingleCurve_FocalLength_Focus025_Zoom0",
			"focus": 0.25,
			"zoom": 0,
			"dataType": "FocalLength",
			"evaluationMethod": "LensFile",
			"expectedResult": [ 3.25, 3 ]
		},
		{
			"name": "SingleCurve_Distortion_Focus075_Zoom1",
			"focus": 0.75,
			"zoom": 1,
			"dataType": "Distortion",
			"evaluationMethod": "LensFile",
			"expectedResult": [ 1.25, 1.5, 0, 0, 0.25 ]
		},
		{
			"name": "SingleCurve_ImageCenter_Focus075_Zoom1",
			"focus": 0.75,
			"zoom": 1,
			"dataType": "ImageCenter",
			"evaluationMethod": "LensFile",
			"expectedResult": [ 5.25, 7.25 ]
		},
		{
			"name": "SingleCurve_FocalLength_Focus075_Zoom1",
			"focus": 0.75,
			"zoom": 1,
			"dataType": "FocalLength",
			"evaluationMethod": "LensFile",
			"expectedResult": [ 8, 6.75 ]
		},
		{
			"name": "DualCurve_Distortion_Focus025_Zoom025",
			"focus": 0.25,
			"zoom": 0.25,
			"dataType": "Distortion",
			"evaluationMethod": "LensFile",
			"expectedResult": [ 0.90625, -0.34375, 0, 0, -0.3046875 ]
		},
		{
			"name": "DualCurve_ImageCenter_Focus025_Zoom025",
			"focus": 0.25,
			"zoom": 0.25,
			"dataType": "ImageCenter",
			"evaluationMethod": "LensFile",
			"expectedResult": [ 6.6953125, 6.484375 ]
		},
		{
			"name": "DualCurve_FocalLength_Focus025_Zoom025",
			"focus": 0.25,
			"zoom": 0.25,
			"dataType": "FocalLength",
			"evaluationMethod": "LensFile",
			"expectedResult": [ 3.6796875, 3.1953125 ]
		},
		{
			"name": "DualCurve_Distortion_Focus075_Zoom025",
			"focus": 0.75,
			"zoom": 0.25,
			"dataType": "Distortion",
			"evaluationMethod": "LensFile",
			"expectedResult": [ 0.40625, 0.65625, 0, 0, -1.2265625 ]
		},
		{
			"name": "DualCurve_ImageCenter_Focus075_Zoom025",
			"focus": 0.75,
			"zoom": 0.25,
			"dataType": "ImageCenter",
			"evaluationMethod": "LensFile",
			"expectedResult": [ 3.7734375, 5.140625 ]
		},
		{
			"name": "DualCurve_FocalLength_Focus075_Zoom025",
			"focus": 0.75,
			"zoom": 0.25,
			"dataType": "FocalLength",
			"evaluationMethod": "LensFile",
			"expectedResult": [ 4.4140625, 5.2734375 ]
		},
		{
			"name": "DualCurve_Distortion_Focus025_Zoom075",
			"focus": 0.25,
			"zoom": 0.75,
			"dataType": "Distortion",
			"evaluationMethod": "LensFile",
			"expectedResult": [ 1.59375, 0.34375, 0, 0, 0.5546875 ]
		},
		{
			"name": "DualCurve_ImageCenter_Focus025_Zoom075",
			"focus": 0.25,
			"zoom": 0.75,
			"dataType": "ImageCenter",
			"evaluationMethod": "LensFile",
			"expectedResult": [ 7.5546875, 7.515625 ]
		},
		{
			"name": "DualCurve_FocalLength_Focus025_Zoom075",
			"focus": 0.25,
			"zoom": 0.75,
			"dataType": "FocalLength",
			"evaluationMethod": "LensFile",
			"expectedResult": [ 5.5703125, 4.0546875 ]
		},
		{
			"name": "DualCurve_Distortion_Focus05_Zoom05",
			"focus": 0.5,
			"zoom": 0.5,
			"dataType": "Distortion",
			"evaluationMethod": "LensFile",
			"expectedResult": [ 1, 0.5, 0, 0, -0.24999994039535522 ]
		},
		{
			"name": "DualCurve_ImageCenter_Focus05_Zoom05",
			"focus": 0.5,
			"zoom": 0.5,
			"dataType": "ImageCenter",
			"evaluationMethod": "LensFile",
			"expectedResult": [ 5.75, 6.5 ]
		},
		{
			"name": "DualCurve_FocalLength_Focus05_Zoom05",
			"focus": 0.5,
			"zoom": 0.5,
			"dataType": "FocalLength",
			"evaluationMethod": "LensFile",
			"expectedResult": [ 5.25, 4.75 ]
		}
	]
}