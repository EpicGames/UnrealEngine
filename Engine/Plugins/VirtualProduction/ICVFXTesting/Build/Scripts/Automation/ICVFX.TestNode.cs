// Copyright Epic Games, Inc. All Rights Reserved.

using Gauntlet;
using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;

namespace ICVFXTest
{
	public class ICVFXTestConfig : UnrealTestConfiguration
	{
		/// Name of the test.
		/// </summary>
		[AutoParamWithNames("ICVFXTest.TestName")]
		public string TestName;

		[AutoParamWithNames("ICVFXTest.MapOverride")]
		public string MapOverride;

		/// <summary>
		/// Total number of loops to run during the test.
		/// </summary>
		[AutoParamWithNames(1, "ICVFXTest.MaxRunCount")]
		public int MaxRunCount;

		/// <summary>
		/// How long to spend in the sandbox, in seconds.
		/// </summary>
		[AutoParamWithNames(30.0f, "ICVFXTest.SoakTime")]
		public float SoakTime;

		/// <summary>
		/// Whether to run the test sequence after the sandbox soak or not.
		/// </summary>
		[AutoParamWithNames(false, "ICVFXTest.SkipTestSequence")]
		public bool SkipTestSequence;

		/// <summary>
		/// If true, appends the stat fps and unitgraph to ExecCmds.
		/// </summary>
		[AutoParamWithNames(false, "ICVFXTest.StatCommands")]
		public bool StatCommands;

		/// <summary>
		/// If true, FPSChart start/end is automatically called for each run.
		/// </summary>
		[AutoParamWithNames(false, "ICVFXTest.FPSChart")]
		public bool FPSChart;

		/// <summary>
		/// If true, MemReport is automatically started each run and captured periodically.
		/// @note Use the ICVFXTest.MemReportInterval CVar to set the interval.
		/// </summary>
		[AutoParamWithNames(false, "ICVFXTest.MemReport")]
		public bool MemReport;

		/// <summary>
		/// If true, video captures are automatically started/ended each run.
		/// @note Requires the platform has an available gameplay media encoder.
		/// </summary>
		[AutoParamWithNames(false, "ICVFXTest.VideoCapture")]
		public bool VideoCapture;

		/// <summary>
		/// Path to the exported ndisplay config to use for testing.
		/// </summary>
		[AutoParamWithNames("", "ICVFXTest.DisplayConfig")]
		public string DisplayConfigPath;

		/// <summary>
		/// Name of the display cluster node to use.
		/// </summary>
		[AutoParamWithNames("", "ICVFXTest.DisplayClusterNodeName")]
		public string DisplayClusterNodeName;

		/// <summary>
		/// The global viewport screen percentage to use. (1-100)
		/// </summary>
		[AutoParamWithNames(100.0f, "ICVFXTest.ViewportScreenPercentage")]
		public float ViewportScreenPercentage;

		/// <summary>
		/// Should run fullscreen
		/// </summary>
		[AutoParamWithNames(false, "ICVFXTest.Fullscreen")]
		public bool Fullscreen;

		/// <summary>
		/// Should run offscreen
		/// </summary>
		[AutoParamWithNames(false, "ICVFXTest.Offscreen")]
		public bool Offscreen;

		/// <summary>
		/// Window X location
		/// </summary>
		[AutoParamWithNames(0, "ICVFXTest.WndLocX")]
		public int WndLocX;

		/// <summary>
		/// Window Y location
		/// </summary>
		[AutoParamWithNames(0, "ICVFXTest.WndLocY")]
		public int WndLocY;

		/// <summary>
		/// Window X size (width)
		/// </summary>
		[AutoParamWithNames(1920, "ICVFXTest.WndResX")]
		public int WndResX;

		/// <summary>
		/// Window Y size (height)
		/// </summary>
		[AutoParamWithNames(1080, "ICVFXTest.WndResY")]
		public int WndResY;

		/// <summary>
		/// Whether to use Lumen or not.
		/// </summary>
		[AutoParamWithNames(false, "ICVFXTest.Lumen")]
		public bool Lumen;

		/// <summary>
		/// Whether to use nanite or not.
		/// </summary>
		[AutoParamWithNames(false, "ICVFXTest.Nanite")]
		public bool Nanite;

		[AutoParamWithNames(false, "ICVFXTest.D3DDebug")]
		public bool D3DDebug;

		[AutoParamWithNames(false, "ICVFXTest.GPUCrashDebugging")]
		public bool GPUCrashDebugging;

		/// <summary>
		/// Sets max number of GPUs to use.
		/// </summary>
		[AutoParamWithNames(1, "ICVFXTest.MaxGPUCount")]
		public int MaxGPUCount;

		/// <summary>
		/// Report type
		/// </summary>
		[AutoParamWithNames("ICVFXTest.SummaryReportType")]
		public string SummaryReportType;

		/// <summary>
		/// Path to the exported ndisplay config to use for testing.
		/// </summary>
		[AutoParamWithNames("ICVFXTest.SummaryReportPath")]
		public string SummaryReportPath;

		/// <summary>
		/// Whether to run with -rdgimmediate
		/// </summary>
		[AutoParamWithNames(false, "ICVFXTest.RDGImmediate")]
		public bool RDGImmediate;

		/// <summary>
		/// Whether to run with -norhithread
		/// </summary>
		[AutoParamWithNames(false, "ICVFXTest.NoRHIThread")]
		public bool NoRHIThread;

		[AutoParamWithNames("ICVFXTest.DisableVirtualShadowMaps")]
		public bool DisableVirtualShadowMaps;

		/// <summary>
		/// Whether to run with unreal insights.
		/// </summary>
		[AutoParamWithNames(false, "ICVFXTest.EnableTrace")]
		public bool EnableTrace;

		/// <summary>
		/// When running with insights, we will overrides the trace file name to put traces in this folder.
		/// </summary>
		[AutoParamWithNames("ICVFXTest.TraceRootFolder")]
		public string TraceRootFolder;

		/// <summary>
		/// Force file name to use for a trace file. If not specified, the default one will be generated.
		/// </summary>
		[AutoParamWithNames("ICVFXTest.TraceFileName")]
		public string TraceFileName;

		/// <summary>
		///  Path to the folder where perf cache data should be stored.
		/// </summary>
		///
		[AutoParamWithNames("ICVFXTest.PerfCacheFolder")]
		public string PerfCacheFolder;

		/// <summary>
		///  Whether to generate the local test reports.
		/// </summary>
		///
		[AutoParamWithNames(false, "ICVFXTest.NoLocalReports")]
		public bool NoLocalReports;

		/// <summary>
		///  Whether to skip uploading gauntlet perf data to the PRS.
		/// </summary>
		///
		[AutoParamWithNames(false, "ICVFXTest.SkipPerfReportServer")]
		public bool SkipPerfReportServer;

		/// <summary>
		///  Override the server import directory for PRS.
		/// </summary>
		///
		[AutoParamWithNames("", "ICVFXTest.PerfReportServerImportDir")]
		public string PerfReportServerImportDir;

		/// <summary>
		/// Whether the server is running in development or not
		/// </summary>
		///
		[AutoParamWithNames(false, "ICFVXTest.IsDevelopment")]
		public bool IsDevelopment;
	}

	public abstract class ICVFXTestNode : UnrealTestNode<ICVFXTestConfig>
	{
		public ICVFXTestNode(UnrealTestContext InContext) : base(InContext)
		{
			TestGuid = Guid.NewGuid();
			Gauntlet.Log.Info("Your Test GUID is :\n" + TestGuid.ToString() + '\n');

			InitHandledErrors();

			ICVFXTestLastLogCount = 0;
		}

		public override bool StartTest(int Pass, int InNumPasses)
		{
			ICVFXTestLastLogCount = 0;
			return base.StartTest(Pass, InNumPasses);
		}

		public class HandledError
		{
			public string ClientErrorString;
			public string GauntletErrorString;

			/// <summary>
			/// String name for the log category that should be used to filter errors. Defaults to null, i.e. no filter.
			/// </summary>
			public string CategoryName;

			// If error is verbose, will output debugging information such as state
			public bool Verbose;

			public HandledError(string ClientError, string GauntletError, string Category, bool VerboseIn = false)
			{
				ClientErrorString = ClientError;
				GauntletErrorString = GauntletError;
				CategoryName = Category;
				Verbose = VerboseIn;
			}
		}

		/// <summary>
		/// List of errors with special-cased gauntlet messages.
		/// </summary>
		public List<HandledError> HandledErrors { get; set; }

		/// <summary>
		/// Guid associated with each test run for ease of differentiation between different runs on same build.
		/// </summary>
		public Guid TestGuid { get; protected set; }

		/// <summary>
		/// Absolute path to Saloon data required during testing and report generation
		/// </summary>
		public static string DataPath { get { return Path.Combine(Globals.UnrealRootDir, "SaloonGame/Test/Gauntlet/Data"); } }

		/// <summary>
		/// Set up the base list of possible expected errors, plus the messages to deliver if encountered.
		/// </summary>
		protected virtual void InitHandledErrors()
		{
			HandledErrors = new List<HandledError>();
		}

		/// <summary>
		/// Line count of the client log messages that have been written to the test logs.
		/// </summary>
		private int ICVFXTestLastLogCount;

		/// <summary>
		/// Periodically called while test is running. Updates logs.
		/// </summary>
		public override void TickTest()
		{
			IAppInstance App = null;

			if (TestInstance.ClientApps == null)
			{
				App = TestInstance.ServerApp;
			}
			else
			{
				if (TestInstance.ClientApps.Length > 0)
				{
					App = TestInstance.ClientApps.First();
				}
			}

			if (App != null)
			{
				UnrealLogParser Parser = new UnrealLogParser(App.StdOut);

				List<string> TestLines = Parser.GetLogChannel("ICVFXTest").ToList();
				
				for (int i = ICVFXTestLastLogCount; i < TestLines.Count; i++)
				{
					if (TestLines[i].StartsWith("LogICVFXTest: Error:"))
					{
						ReportError(TestLines[i]);
					}
					else if (TestLines[i].StartsWith("LogICVFXTest: Warning:"))
					{
						ReportWarning(TestLines[i]);
					}
					else
					{
						Log.Info(TestLines[i]);
					}
				}

				ICVFXTestLastLogCount = TestLines.Count;
			}

			base.TickTest();
		}

		protected override UnrealProcessResult GetExitCodeAndReason(StopReason InReason, UnrealLog InLogSummary, UnrealRoleArtifacts InArtifacts, out string ExitReason, out int ExitCode)
		{
			// Check for login failure
			UnrealLogParser Parser = new UnrealLogParser(InArtifacts.AppInstance.StdOut);

			ExitReason = "";
			ExitCode = -1;

			foreach (HandledError ErrorToCheck in HandledErrors)
			{
				string[] MatchingErrors = Parser.GetErrors(ErrorToCheck.CategoryName).Where(E => E.Contains(ErrorToCheck.ClientErrorString)).ToArray();
				if (MatchingErrors.Length > 0)
				{
					ExitReason = string.Format("Test Error: {0} {1}", ErrorToCheck.GauntletErrorString, ErrorToCheck.Verbose ? "\"" + MatchingErrors[0] + "\"" : "");
					ExitCode = -1;
					return UnrealProcessResult.TestFailure;
				}
			}

			return base.GetExitCodeAndReason(InReason, InLogSummary, InArtifacts, out ExitReason, out ExitCode);
		}


		/// <summary>
		/// CreateReport() happens near the end of StopTest after SaveArtifacts(). Override this function within your test to set up external reporting.
		/// Include a base call so that ReportToDashboard() is called appropriately.
		/// </summary>
		public override ITestReport CreateReport(TestResult Result, UnrealTestContext Context, UnrealBuildSource Build, IEnumerable<UnrealRoleResult> Artifacts, string ArtifactPath)
		{
			return base.CreateReport(Result, Context, Build, RoleResults, ArtifactPath);
		}
	}
}