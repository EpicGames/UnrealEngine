// Copyright Epic Games, Inc. All Rights Reserved.

using Gauntlet;
using Microsoft.Extensions.Logging;
using System.Collections.Generic;

using Log = EpicGames.Core.Log;

namespace ICVFXTest
{
	/// <summary>
	/// CI testing
	/// </summary>
	public class AutoTest : ICVFXTestNode
	{
		private static ILogger Logger => Log.Logger;
		
		public AutoTest(Gauntlet.UnrealTestContext InContext)
			: base(InContext)
		{
		}
		public virtual int GetMaxGPUCount()
		{
			return base.GetConfiguration().MaxGPUCount;
		}

		public virtual string GetTestSuffix()
		{
			return "Base";
		}

		public virtual bool IsLumenEnabled()
		{
			return base.GetConfiguration().Lumen;
		}
		public virtual bool UseVulkan()
		{
			return false;
		}

		public virtual bool UseNanite()
		{
			return base.GetConfiguration().Nanite;
		}

		public ICVFXTestConfig GetBaseConfiguration()
		{
			return base.GetConfiguration();
		}

		public virtual string GetDisplayConfigPath()
		{
			return base.GetConfiguration().DisplayConfigPath;
		}

		public virtual string GetDisplayClusterNode()
		{
			return base.GetConfiguration().DisplayClusterNodeName;
		}

		public virtual string GetDisplayClusterUAssetPath(in string NDisplayJsonFile)
		{
			return "";
		}

		public override ICVFXTestConfig GetConfiguration()
		{
			ICVFXTestConfig Config = base.GetConfiguration();
			Config.MaxDuration = Context.TestParams.ParseValue("MaxDuration", 60 * 60);  // 1 hour max
			
			UnrealTestRole ClientRole = Config.RequireRole(UnrealTargetRole.Client);
			ClientRole.Controllers.Add("ICVFXTestControllerAutoTest");

			ClientRole.CommandLineParams.AddOrAppendParamValue("DPCVars", "Slate.bAllowNotifications=0");
			ClientRole.CommandLineParams.AddOrAppendParamValue("DPCVars", "p.Chaos.Solver.Deterministic=1");

			ClientRole.CommandLineParams.AddOrAppendParamValue("execcmds", "ICVFXTest.MaxRunCount " + Config.MaxRunCount);
			ClientRole.CommandLineParams.AddOrAppendParamValue("execcmds", "ICVFXTest.SoakTime " + Config.SoakTime);

			if (Config.EnableTrace && Config.TraceRootFolder.Length != 0)
			{
				// If no file name specified, generate default name
				if (string.IsNullOrEmpty(Config.TraceFileName))
				{
					// Original approach
					ClientRole.CommandLineParams.AddOrAppendParamValue("execcmds", "ICVFXTest.TraceFileName " + $"{Config.TraceRootFolder}/{GetTestSuffix()}.utrace");
				}
				else
				{
					// Explicit file name
					ClientRole.CommandLineParams.AddOrAppendParamValue("execcmds", "ICVFXTest.TraceFileName " + $"{Config.TraceRootFolder}/{Config.TraceFileName}");
				}
			}

			ClientRole.CommandLineParams.AddOrAppendParamValue("execcmds", "r.nanite " + (UseNanite() ? 1 : 0));

			// Todo: verify this is actually used in an ndisplay context and if we need to set an override option as well.
			ClientRole.CommandLineParams.AddOrAppendParamValue("execcmds", "r.ScreenPercentage " + Config.ViewportScreenPercentage);

			bool bLumenEnabled = IsLumenEnabled();
			ClientRole.CommandLineParams.AddOrAppendParamValue("execcmds", "r.RayTracing " + (bLumenEnabled ? 1 : 0));
			ClientRole.CommandLineParams.AddOrAppendParamValue("execcmds", "r.DynamicGlobalIlluminationMethod " + (bLumenEnabled ? 1 : 0));
			ClientRole.CommandLineParams.AddOrAppendParamValue("execcmds", "r.ReflectionMethod " + (bLumenEnabled? 2 : 0));
			ClientRole.CommandLineParams.AddOrAppendParamValue("execcmds", "r.Lumen.HardwareRayTracing " + (bLumenEnabled ? 1 : 0));
			ClientRole.CommandLineParams.AddOrAppendParamValue("execcmds", "r.Lumen.HardwareRayTracing.LightingMode " + (bLumenEnabled ? 2 : 0));

			// Temporary until we figure out why we hang
			ClientRole.CommandLineParams.AddOrAppendParamValue("execcmds", "FX.AllowGPUParticles 0");
			ClientRole.CommandLineParams.AddOrAppendParamValue("execcmds", "r.Shadow.Virtual.Enable " + (Config.DisableVirtualShadowMaps ? 0 : 1));
			ClientRole.CommandLineParams.AddOrAppendParamValue("execcmds", "DisableAllScreenMessages");

			ClientRole.CommandLineParams.Add("MaxGPUCount=" + GetMaxGPUCount());

			if (Config.SkipTestSequence)
			{
				ClientRole.CommandLineParams.AddOrAppendParamValue("execcmds", "ICVFXTest.SkipTestSequence true");
			}

			if (Config.StatCommands)
			{
				ClientRole.CommandLineParams.AddOrAppendParamValue("execcmds", "stat fps");
				ClientRole.CommandLineParams.AddOrAppendParamValue("execcmds", "stat unitgraph");
			}
			
			if (Config.FPSChart)
			{
				ClientRole.CommandLineParams.Add("ICVFXTest.FPSChart");
			}
			
			if (Config.MemReport)
			{
				ClientRole.CommandLineParams.Add("ICVFXTest.MemReport");
			}

			if (Config.VideoCapture)
			{
				ClientRole.CommandLineParams.Add("ICVFXTest.VideoCapture");
			}

			if (Config.MapOverride?.Length != 0)
			{
				ClientRole.MapOverride = Config.MapOverride;
				Logger.LogInformation($"Map Override: { Config.MapOverride }");
			}

			if (Config.D3DDebug)
			{
				ClientRole.CommandLineParams.Add("d3ddebug");
				Gauntlet.Log.Info("Running with D3DDebug");
			}

			if (Config.GPUCrashDebugging)
			{
				ClientRole.CommandLineParams.Add("gpucrashdebugging");
				Gauntlet.Log.Info("Running with GPUCrashDebugging");
			}

			string DisplayConfigPath = Config.DisplayConfigPath;
			if (string.IsNullOrEmpty(DisplayConfigPath))
			{
				DisplayConfigPath = GetDisplayConfigPath();
			}

			if (DisplayConfigPath?.Length != 0)
			{
				string DisplayClusterUObjectPath = GetDisplayClusterUAssetPath(DisplayConfigPath);

				Logger.LogInformation($"Using -> {DisplayClusterUObjectPath}.uasset");
				if (!string.IsNullOrEmpty(DisplayClusterUObjectPath))
				{
					// Needed for the test runner to figure out which display cluster root actor to move in the scene.
					ClientRole.CommandLineParams.AddOrAppendParamValue("ICVFXTest.DisplayClusterUAssetPath", DisplayClusterUObjectPath);
				}

				ClientRole.CommandLineParams.AddOrAppendParamValue("dc_cfg", DisplayConfigPath);
			}

			string DisplayConfigNode = Config.DisplayClusterNodeName;
			if (string.IsNullOrEmpty(DisplayConfigNode))
			{
				DisplayConfigNode = GetDisplayClusterNode();
			}

			if (DisplayConfigNode?.Length != 0)
			{
				ClientRole.CommandLineParams.AddOrAppendParamValue("dc_node", DisplayConfigNode);
			}

			if (UseVulkan())
			{
				ClientRole.CommandLineParams.Add("vulkan");
			}
			else
			{
				ClientRole.CommandLineParams.Add("dx12");
			}

			if (Config.RDGImmediate)
			{
				ClientRole.CommandLineParams.Add("rdgimmediate");
			}
			
			if (Config.NoRHIThread)
			{
				ClientRole.CommandLineParams.Add("norhithread");
			}

			if (Config.EnableTrace)
			{
				// Start as late as possible to reduce trace file size.s
				ClientRole.CommandLineParams.Add("traceautostart=0");
				ClientRole.CommandLineParams.Add("trace=cpu,counters,gpu,bookmark");
			}

			ClientRole.CommandLineParams.Add("unattended");
			ClientRole.CommandLineParams.Add("NoScreenMessages");
			ClientRole.CommandLineParams.Add("handleensurepercent=0");
			ClientRole.CommandLineParams.Add("messaging");
			ClientRole.CommandLineParams.Add("dc_cluster");
			ClientRole.CommandLineParams.Add("nosplash");
			ClientRole.CommandLineParams.Add("NoLoadingScreen");
			ClientRole.CommandLineParams.Add("fixedseed");
			ClientRole.CommandLineParams.Add("NoVerifyGC");
			ClientRole.CommandLineParams.Add("noxrstereo");
			ClientRole.CommandLineParams.Add("xrtrackingonly");
			ClientRole.CommandLineParams.Add("RemoteControlIsHeadless");
			ClientRole.CommandLineParams.Add("UDPMESSAGING_TRANSPORT_MULTICAST=\"230.0.0.1:6666\"");
			ClientRole.CommandLineParams.Add("UDPMESSAGING_TRANSPORT_UNICAST=\"127.0.0.1:0\"");
			ClientRole.CommandLineParams.Add("UDPMESSAGING_TRANSPORT_STATIC=\"127.0.0.1:9030\"");
			ClientRole.CommandLineParams.Add("ini:Engine:[/Script/Engine.Engine]:GameEngine=/Script/DisplayCluster.DisplayClusterGameEngine,[/Script/Engine.Engine]:GameViewportClientClassName=/Script/DisplayCluster.DisplayClusterViewportClient,[/Script/Engine.UserInterfaceSettings]:bAllowHighDPIInGameMode=True");
			ClientRole.CommandLineParams.Add("ini:Game:[/Script/EngineSettings.GeneralProjectSettings]:bUseBorderlessWindow=True");
			ClientRole.CommandLineParams.Add("DisablePlugins=\"RenderDocPlugin\"");
			ClientRole.CommandLineParams.Add("dc_dev_mono");

			if (Config.Fullscreen)
			{
				ClientRole.CommandLineParams.Add("fullscreen");
			}
			else
			{
				ClientRole.CommandLineParams.Add($"WinX={Config.WndLocX}");
				ClientRole.CommandLineParams.Add($"WinY={Config.WndLocY}");
				ClientRole.CommandLineParams.Add($"ResX={Config.WndResX}");
				ClientRole.CommandLineParams.Add($"ResY={Config.WndResY}");
				ClientRole.CommandLineParams.AddOrAppendParamValue("execcmds", $"r.setres {Config.WndResX}x{Config.WndResY}");

				if (Config.Offscreen)
				{
					ClientRole.CommandLineParams.Add("RenderOffscreen");
				}
				else
				{
					ClientRole.CommandLineParams.Add("forceres");
					ClientRole.CommandLineParams.Add("windowed");
				}
			}

			Gauntlet.Log.Info(ClientRole.CommandLineParams.ToString());

			return Config;
		}

		protected override void InitHandledErrors()
		{
			base.InitHandledErrors();
			HandledErrors.Add(new HandledError("AutoTest", "AutoTest failure:", "LogICVFXTest", true));
		}

		public override ITestReport CreateReport(TestResult Result, UnrealTestContext Context, UnrealBuildSource Build, IEnumerable<UnrealRoleResult> Artifacts, string ArtifactPath)
		{
			return base.CreateReport(Result, Context, Build, Artifacts, ArtifactPath);
		}
	}
}
