// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using EpicGames.Core;
using Gauntlet;
using Microsoft.Extensions.Logging;
using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using UnrealBuildBase;
using UnrealBuildTool;
using Log = EpicGames.Core.Log;

namespace ICVFXTest
{
    public interface ICsvProcessor
    {
        List<CsvRecord> ReadCsv(string filePath, List<string> stats);
    }

    public class CsvRecord
    {
        public string BuildVersion { get; set; } = string.Empty;
        public Dictionary<string, double> Stats { get; set; } = new Dictionary<string, double>();
    }

    public class CsvProcessor : ICsvProcessor
    {
        private static ILogger Logger => Log.Logger;

        public List<CsvRecord> ReadCsv(string filePath, List<string> stats)
        {
            Logger.LogInformation("Reading CSV file from {FilePath}", filePath);

            var records = new List<CsvRecord>();

            try
            {
                using (var reader = new StreamReader(filePath))
                {
                    var headers = reader.ReadLine()?.Split(',');

                    if (headers == null)
                    {
                        Logger.LogWarning("CSV file at {FilePath} has no headers.", filePath);
                        return records;
                    }

                    string line;
                    while ((line = reader.ReadLine()) != null)
                    {
                        var values = line.Split(',');

                        var record = new CsvRecord
                        {
                            BuildVersion = values[0]?.Split('-').Last() ?? string.Empty
                        };

                        for (int i = 0; i < headers.Length; i++)
                        {
                            if (stats.Contains(headers[i]))
                            {
                                if (double.TryParse(values[i], out var statValue))
                                {
                                    record.Stats[headers[i]] = statValue;
                                }
                                else
                                {
                                    Logger.LogWarning("Failed to parse value for {Stat} at line: {Line}", headers[i], line);
                                }
                            }
                        }
                        records.Add(record);
                    }
                }

                Logger.LogInformation("Successfully read {RecordCount} records from CSV.", records.Count);
            }
            catch (Exception ex)
            {
                Logger.LogError(ex, "Error reading CSV file at {FilePath}", filePath);
            }

            return records;
        }
    }

    public interface ICsvToSvgRunner
    {
        void RunCsvToSvg(string toolPath, List<CsvRecord> records, string outputDir, string stat);
    }

    public class CsvToSvgRunner : ICsvToSvgRunner
    {
        private static ILogger Logger => Log.Logger;

        public void RunCsvToSvg(string toolPath, List<CsvRecord> records, string outputDir, string stat)
        {
            Logger.LogInformation("Running CSVToSVG for stat {Stat}", stat);

            var sanitizedStat = SanitizeFileName(stat);
            var tempCsvPath = Path.Combine(outputDir, $"{sanitizedStat}_temp.csv");
            var outputSvgPath = Path.Combine(outputDir, $"{sanitizedStat}.svg");

            try
            {
                using (var writer = new StreamWriter(tempCsvPath))
                {
                    writer.WriteLine("Frame,BuildVersion," + stat);
                    int frameNumber = 1;
                    foreach (var record in records)
                    {
                        writer.WriteLine($"{frameNumber},{record.BuildVersion},{record.Stats[stat]}");
                        frameNumber++;
                    }
                }

                string arguments = $"-csv \"{tempCsvPath}\" -o \"{outputSvgPath}\" -stats \"{stat}\" -interactive -skipRows 1";

                var processStartInfo = new ProcessStartInfo
                {
                    FileName = toolPath,
                    Arguments = arguments,
                    RedirectStandardOutput = true,
                    RedirectStandardError = true,
                    UseShellExecute = false,
                    CreateNoWindow = true
                };

                using var process = new Process
                {
                    StartInfo = processStartInfo
                };

                process.OutputDataReceived += (sender, e) =>
                {
                    if (!string.IsNullOrEmpty(e.Data))
                    {
                        Logger.LogInformation("CSVToSVG Output: {Output}", e.Data);
                    }
                };

                process.ErrorDataReceived += (sender, e) =>
                {
                    if (!string.IsNullOrEmpty(e.Data))
                    {
                        Logger.LogError("CSVToSVG Error: {Error}", e.Data);
                    }
                };

                process.Start();
                process.BeginOutputReadLine();
                process.BeginErrorReadLine();
                process.WaitForExit();

                if (File.Exists(outputSvgPath))
                {
                    Logger.LogInformation("{OutputSvgPath} created successfully.", outputSvgPath);
                }
                else
                {
                    Logger.LogError("Failed to create {OutputSvgPath}.", outputSvgPath);
                }

                File.Delete(tempCsvPath);
            }
            catch (Exception ex)
            {
                Logger.LogError(ex, "Error running CSVToSVG for stat {Stat}", stat);
            }

            // TODO-Imaad - Modify SVG file after creation and replace Frames with transformed CL
        }

        public static string SanitizeFileName(string fileName)
        {
            var invalidChars = Path.GetInvalidFileNameChars();
            return string.Concat(fileName.Select(c => invalidChars.Contains(c) ? '_' : c));
        }
    }

    public interface IHtmlReportGenerator
    {
        void GenerateReport(string outputDir, List<string> stats);
    }

    public class HtmlReportGenerator : IHtmlReportGenerator
    {
        private static ILogger Logger => Log.Logger;

        public void GenerateReport(string outputDir, List<string> stats)
        {
            Logger.LogInformation("Generating HTML report in {OutputDir}", outputDir);

            var reportPath = Path.Combine(outputDir, "report.html");

            try
            {
                using (var writer = new StreamWriter(reportPath))
                {
                    writer.WriteLine("<html><head>");
                    writer.WriteLine("<style>");
                    writer.WriteLine("body { font-family: Arial, sans-serif; margin: 0; padding: 0; background-color: #f4f4f9; color: #333; }");
                    writer.WriteLine(".container { padding: 20px; }");
                    writer.WriteLine("h1 { text-align: center; color: #333; }");
                    writer.WriteLine("select { margin: 10px 0; padding: 5px; font-size: 16px; width: 200px; }");
                    writer.WriteLine(".graph-container { display: none; text-align: center; }");
                    writer.WriteLine(".dropdown-container { text-align: center; margin-bottom: 20px; }");
                    writer.WriteLine("</style>");
                    writer.WriteLine("<script>");
                    writer.WriteLine("function showGraph() {");
                    writer.WriteLine("var selectedStat = document.getElementById('statSelector').value;");
                    writer.WriteLine("var graphs = document.getElementsByClassName('graph-container');");
                    writer.WriteLine("for (var i = 0; i < graphs.length; i++) { graphs[i].style.display = 'none'; }");
                    writer.WriteLine("document.getElementById(selectedStat).style.display = 'block';");
                    writer.WriteLine("}");
                    writer.WriteLine("</script>");
                    writer.WriteLine("</head><body>");
                    writer.WriteLine("<div class='container'>");
                    writer.WriteLine("<h1><a href='https://www.google.com' style='text-decoration: none; color: #333;'>Saloon Automated Tests - Stat History</a></h1>");
                    writer.WriteLine("<div class='dropdown-container'>");
                    writer.WriteLine("<label for='testTypeSelector'>Select Test Type:</label>");
                    writer.WriteLine("<select id='testTypeSelector'>");
                    writer.WriteLine("<option value='default'>Default</option>");
                    writer.WriteLine("</select>");
                    writer.WriteLine("</div>");
                    writer.WriteLine("<div class='dropdown-container'>");
                    writer.WriteLine("<label for='statSelector'>Select a stat:</label>");
                    writer.WriteLine("<select id='statSelector' onchange='showGraph()'>");

                    foreach (var stat in stats)
                    {
                        var sanitizedStat = CsvToSvgRunner.SanitizeFileName(stat);
                        writer.WriteLine($"<option value='{sanitizedStat}'>{stat}</option>");
                    }

                    writer.WriteLine("</select>");
                    writer.WriteLine("</div>");
                    writer.WriteLine("<div id='graphs'>");

                    foreach (var stat in stats)
                    {
                        var sanitizedStat = CsvToSvgRunner.SanitizeFileName(stat);
                        var svgPath = $"{sanitizedStat}.svg";
                        writer.WriteLine($"<div id='{sanitizedStat}' class='graph-container'>");
                        writer.WriteLine($"<h2>{stat}</h2>");
                        writer.WriteLine($"<object type=\"image/svg+xml\" data=\"{svgPath}\" width=\"1800\" height=\"550\"></object>");
                        writer.WriteLine("</div>");
                    }

                    writer.WriteLine("</div>");
                    writer.WriteLine("</div>");
                    writer.WriteLine("<script>document.getElementById('statSelector').value = '{CsvToSvgRunner.SanitizeFileName(stats[0])}'; showGraph();</script>");
                    writer.WriteLine("</body></html>");
                }

                Logger.LogInformation("HTML report generated successfully at {ReportPath}.", reportPath);
            }
            catch (Exception ex)
            {
                Logger.LogError(ex, "Error generating HTML report at {ReportPath}.", reportPath);
            }
        }
    }
}
