// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGame;
using Gauntlet;
using System.Collections.Generic;

namespace ICVFXTest
{
	/// <summary>
	/// CI testing
	/// </summary>
	public class MemoryReport : AutoTest
	{
		public MemoryReport(Gauntlet.UnrealTestContext InContext)
			: base(InContext)
		{
		}

		public override ICVFXTestConfig GetConfiguration()
		{
			ICVFXTestConfig Config = base.GetConfiguration();

			UnrealTestRole ClientRole = Config.RequireRole(UnrealTargetRole.Client);
			ClientRole.CommandLineParams.Add("ICVFXTest.MemReport");
			return Config;
		}
	}
}
