// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using EpicGames.Core;
using Gauntlet;
using ICVFXTest.Switchboard;
using Microsoft.Extensions.Logging;
using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Text.Json;
using System.Text.Json.Serialization;
using UnrealBuildBase;
using UnrealBuildTool;
using Log = EpicGames.Core.Log;

namespace ICVFXTest
{
	namespace NDisplay
	{
		public class NDisplayInnerObject
		{
			[JsonPropertyName("assetPath")]
			public string AssetPath { get; set; }
		}

		public class NDisplayConfig
		{
			[JsonPropertyName("nDisplay")]
			public NDisplayInnerObject InnerConfig { get; set; }
		}
	}

	namespace Switchboard
	{
		public class NDisplayDeviceSetting
		{
			[JsonPropertyName("ndisplay_cfg_file")]
			public string DisplayConfigFile { get; set; }

			[JsonPropertyName("primary_device_name")]
			public string PrimaryDeviceName { get; set; }
		}

		public class NdisplaySettings
		{
			[JsonPropertyName("settings")]
			public NDisplayDeviceSetting Settings { get; set; }
		}

		public class NDisplayConfig
		{
			[JsonPropertyName("nDisplay")]
			public NdisplaySettings DeviceSettings { get; set; }
		}

		/// <summary>
		/// Represents a config output by switchboard.
		/// </summary>
		public class SwitchboardConfig
		{
			public string ConfigName { get; set; }

			[JsonPropertyName("project_name")]
			public string ProjectName { get; set; }

			[JsonPropertyName("uproject")]
			public string ProjectPath { get; set; }

			[JsonPropertyName("devices")]
			public NDisplayConfig Devices { get; set; }
		}


		public class UserSettings
		{
			[JsonPropertyName("config")]
			public string LastUsedConfig { get; set; }

			[JsonPropertyName("last_browsed_path")]
			public string LastBrowsedPath { get; set; }
		}
	}

	/// <summary>
	/// CI testing
	/// </summary>
	public class PerformanceReport : AutoTest
	{
		private static ILogger Logger => Log.Logger;
		private string OriginalBuildName;
		private string OverrideDisplayConfigPath;
		private string OverrideDisplayClusterNode;

		// Temporary directory for the CSV file
		private DirectoryInfo TempPerfCSVDir
		{
			get
			{
				// When running with node ID specified, embed the ID into the directory name
				string NodeId = GetConfiguration().DisplayClusterNodeName;
				string ReportDirName = string.IsNullOrEmpty(NodeId) ? "PerfReportCSVs" : "PerfReportCSVs_" + NodeId;
				return new DirectoryInfo(Path.Combine(Unreal.RootDirectory.FullName, "GauntletTemp", ReportDirName));
			}
		}

		// Generates cmdline parameter to configure the neccessary type of report (single line embedding)
		private string ReportTypeCmdLineParam
		{
			get
			{
				string CfgReportType = GetConfiguration().SummaryReportType;
				return string.IsNullOrEmpty(CfgReportType) ? string.Empty : $"-reportType {CfgReportType}";
			}
		}

		public PerformanceReport(UnrealTestContext InContext)
			: base(InContext)
		{
			FindSwitchboardConfigs();

			// Save off the BuildName to prevent a mismatch in CreateReport
			OriginalBuildName = Context.TestParams.ParseValue("ICVFXTest.BuildName", null);
			Logger.LogInformation("Setting OriginalBuildName to {OriginalBuildName}", OriginalBuildName);
		}

		public override ICVFXTestConfig GetConfiguration()
		{
			ICVFXTestConfig Config = base.GetConfiguration();
			UnrealTestRole ClientRole = Config.RequireRole(UnrealTargetRole.Client);

			// Add CSV metadata
			List<string> CsvMetadata = new List<string>
			{
				"testname=" + Config.TestName,
				"gauntletTestType=AutoTest",
				"gauntletSubTest=Performance",
				"testBuildIsPreflight=" + (ReportGenUtils.IsTestingPreflightBuild(OriginalBuildName) ? "1" : "0"),
				"testBuildVersion=" + OriginalBuildName,
				"config=" + (Config.IsDevelopment ? "development" : "test")
			};

			if (!string.IsNullOrEmpty(Context.BuildInfo.Branch) && Context.BuildInfo.Changelist != 0)
			{
				CsvMetadata.Add("branch=" + Context.BuildInfo.Branch);
				CsvMetadata.Add("changelist=" + Context.BuildInfo.Changelist);
			}

			// Set CL parameters
			ClientRole.CommandLineParams.Add("csvGpuStats");
			ClientRole.CommandLineParams.Add("csvMetadata", "\"" + String.Join(",", CsvMetadata) + "\"");
			ClientRole.CommandLineParams.AddOrAppendParamValue("execcmds", "t.FPSChart.DoCSVProfile 1");
			ClientRole.CommandLineParams.AddOrAppendParamValue("execcmds", "t.FPSChart.OpenFolderOnDump 0");
			ClientRole.CommandLineParams.Add("ICVFXTest.FPSChart");

			return Config;
		}

		protected override void InitHandledErrors()
		{
			base.InitHandledErrors();
		}

		public override string GetDisplayClusterUAssetPath(in string NDisplayJsonFile)
		{
			var NDisplayJsonFullPath = Path.Combine(Unreal.RootDirectory.FullName, NDisplayJsonFile);
			string Text = File.ReadAllText(NDisplayJsonFullPath);
			NDisplay.NDisplayConfig DisplayConfig = JsonSerializer.Deserialize<NDisplay.NDisplayConfig>(Text);
			return DisplayConfig.InnerConfig.AssetPath;
		}

		void FindSwitchboardConfigs()
		{
			var SwitchboardConfigsFolder = Path.Combine(Unreal.RootDirectory.FullName, "Engine", "Plugins", "VirtualProduction", "Switchboard", "Source", "Switchboard", "configs");
			if (Directory.Exists(SwitchboardConfigsFolder))
			{
				string[] FileEntries = Directory.GetFiles(SwitchboardConfigsFolder);

				List<SwitchboardConfig> ConfigsForProject = new List<SwitchboardConfig>();

				string LastConfigName = "";

				foreach (string ConfigFile in FileEntries)
				{
					string Text = File.ReadAllText(ConfigFile);

					if (ConfigFile.EndsWith("user_settings.json"))
					{
						var UserConfig = JsonSerializer.Deserialize<Switchboard.UserSettings>(Text);
						LastConfigName = UserConfig.LastUsedConfig;
						Logger.LogInformation($"Last config used: {LastConfigName}");
						continue;
					}

					var Config = JsonSerializer.Deserialize<Switchboard.SwitchboardConfig>(Text);
					Config.ConfigName = Path.GetFileName(ConfigFile);

					if (Config.ProjectName == Context.Options.Project)
					{
						ConfigsForProject.Add(Config);
					}
				}
				
				Logger.LogInformation($"Detected {ConfigsForProject.Count} switchboard configs for this project.");

				foreach (SwitchboardConfig Config in ConfigsForProject)
				{
					Logger.LogInformation($"\t{Config.ConfigName}");

					if (!String.IsNullOrEmpty(LastConfigName) && Config.ConfigName == LastConfigName)
					{
						Logger.LogInformation($"Last NDisplay Config used: {Config.Devices.DeviceSettings.Settings.DisplayConfigFile}");
						Logger.LogInformation($"Found DisplayConfigFile {Config.Devices.DeviceSettings.Settings.DisplayConfigFile}");
						OverrideDisplayConfigPath = Config.Devices.DeviceSettings.Settings.DisplayConfigFile;

						Logger.LogInformation($"Using autodetected DisplayConfig Node: {Config.Devices.DeviceSettings.Settings.PrimaryDeviceName}");
						OverrideDisplayClusterNode = Config.Devices.DeviceSettings.Settings.PrimaryDeviceName;
						break;
					}
				}

				if ((string.IsNullOrEmpty(OverrideDisplayConfigPath) || string.IsNullOrEmpty(OverrideDisplayClusterNode)) && ConfigsForProject.Count() != 0)
				{
					// Fallback to first config that we found from switchboard.
					SwitchboardConfig FirstConfig = ConfigsForProject.First();

					if (string.IsNullOrEmpty(OverrideDisplayConfigPath))
					{
						Logger.LogInformation($"Using autodetected DisplayConfigFile {FirstConfig.Devices.DeviceSettings.Settings.DisplayConfigFile}");
						OverrideDisplayConfigPath = FirstConfig.Devices.DeviceSettings.Settings.DisplayConfigFile;
					}

					if (string.IsNullOrEmpty(OverrideDisplayClusterNode))
					{
						Logger.LogInformation($"Using autodetected DisplayConfig Node: {FirstConfig.Devices.DeviceSettings.Settings.PrimaryDeviceName}");
						OverrideDisplayClusterNode = FirstConfig.Devices.DeviceSettings.Settings.PrimaryDeviceName;
					}
				}
			}

			if (string.IsNullOrEmpty(OverrideDisplayClusterNode))
			{
				// Fallback to node_0 if we don't find anything.
				OverrideDisplayClusterNode = "node_0";
			}
		}

		public override string GetDisplayConfigPath()
		{
			return OverrideDisplayConfigPath;
		}

		public override string GetDisplayClusterNode()
		{
			return OverrideDisplayClusterNode;
		}

		// Find the newest csv file and get its directory
		private String GetNewestCSVDir(UnrealTargetPlatform Platform, string ArtifactPath, string TempDir) 
		{
			// All CSV paths to look through
			var CSVsPaths = new[]
			{
				Path.Combine(ArtifactPath, "EditorGame", "Profiling", "FPSChartStats"),
				Path.Combine(ArtifactPath, "EditorGame", "Settings", $"{Context.Options.Project}", "Saved", "Profiling", "FPSChartStats"),
				Path.Combine(TempDir, "DeviceCache", Platform.ToString(), TestInstance.ClientApps[0].Device.ToString(), "UserDir")
			};

			// Check all paths for potential CSVs
			var DiscoveredCSVs = new List<string>();
			foreach (var CSVsPath in CSVsPaths)
			{
				if (Directory.Exists(CSVsPath))
				{
					DiscoveredCSVs.AddRange(
						from CsvFile in Directory.GetFiles(CSVsPath, "*.csv", SearchOption.AllDirectories)
						where CsvFile.Contains("csvprofile", StringComparison.InvariantCultureIgnoreCase)
						select CsvFile);
				}
			}

			if (DiscoveredCSVs.Count == 0)
			{
				Logger.LogError($"Test completed successfully but no CSV profiling results were found. Searched paths were:\r\n  {string.Join("\r\n  ", CSVsPaths.Select(s => $"\"{s}\""))}");
				return null;
			}

			// Find the newest csv file and get its directory
			// (PerfReportTool will only output cached data in -csvdir mode)
			var NewestFile =
				(from CsvFile in DiscoveredCSVs
				 let Timestamp = File.GetCreationTimeUtc(CsvFile)
				 orderby Timestamp descending
				 select CsvFile).First();
			var NewestDir = Path.GetDirectoryName(NewestFile);

			return NewestDir;
		}

		/// <summary>
		/// Stores perf data in local perf cache which can be uploaded to PRS, 
		/// and produces a detailed csv report using PerfReportTool using the data the cache contains.
		/// </summary>
		private void GeneratePerfReport(UnrealTargetPlatform Platform, string ArtifactPath, string TempDir)
		{
			var ReportCacheDir = GetConfiguration().PerfCacheFolder;

			if (GetTestSuffix().Length != 0)
			{
				ReportCacheDir += "_" + GetTestSuffix(); // We don't want to mix test results
			}

			var ToolPath = FileReference.Combine(Unreal.EngineDirectory, "Binaries", "DotNET", "CsvTools", "PerfreportTool.exe");
			if (!FileReference.Exists(ToolPath))
			{
				Logger.LogError($"Failed to find perf report utility at this path: \"{ToolPath}\".");
				return;
			}
			var ReportConfigDir = Path.Combine(Unreal.RootDirectory.FullName, "Engine", "Plugins", "VirtualProduction", "ICVFXTesting", "Build", "Scripts", "PerfReport");
			var ReportPath = Path.Combine(ArtifactPath, "Reports", "Performance");
			var NewestDir = GetNewestCSVDir(Platform, ArtifactPath, TempDir);

			if(string.IsNullOrEmpty(NewestDir)) { 
				Logger.LogWarning("Failed to execute perf report tool");
				return;
			}

			Logger.LogInformation($"Using perf report cache directory \"{ReportCacheDir}\".");
			Logger.LogInformation($"Using perf report output directory \"{ReportPath}\".");
			Logger.LogInformation($"Using csv results directory \"{NewestDir}\". Generating historic perf report data...");

			// Make sure the cache and output directories exist
			if (!Directory.Exists(ReportCacheDir))
			{
				try { Directory.CreateDirectory(ReportCacheDir); }
				catch (Exception Ex)
				{
					Logger.LogError($"Failed to create perf report cache directory \"{ReportCacheDir}\". {Ex}");
					return;
				}
			}
			if (!Directory.Exists(ReportPath))
			{
				try { Directory.CreateDirectory(ReportPath); }
				catch (Exception Ex)
				{
					Logger.LogError($"Failed to create perf report output directory \"{ReportPath}\". {Ex}");
					return;
				}
			}

			// Win64 is actually called "Windows" in csv profiles
			var PlatformNameFilter = Platform == UnrealTargetPlatform.Win64 ? "Windows" : $"{Platform}";

			// Generate HTML report
			var Args = new[]
			{
				$"-csvdir \"{NewestDir}\"",
				$"-o \"{ReportPath}\"",
				$"-reportxmlbasedir \"{ReportConfigDir}\"",
				$"-summaryTableCache \"{ReportCacheDir}\"",
				$"-metadatafilter platform=\"{PlatformNameFilter}\"",
				$"{ReportTypeCmdLineParam}",
				"-searchpattern csvprofile*"
			};

			var ArgStr = string.Join(" ", Args);

			// Produce the detailed report, and update the perf cache
			CommandUtils.RunAndLog(ToolPath.FullName, ArgStr, out int ErrorCode);
			if (ErrorCode != 0)
			{
				Logger.LogError($"PerfReportTool returned error code \"{ErrorCode}\" while generating detailed report.");
			}

			// Generates HTML & CSV reports to Reports/SaloonPerf
			void HistoricReport(string Name, IEnumerable<string> Filter)
			{
				// Generate HTML report
				var Args = new[]
				{
					$"-summarytablecachein \"{ReportCacheDir}\"",
					$"-summaryTableFilename \"{Name  + GetTestSuffix()}.html\"",
					$"-reportxmlbasedir \"{ReportConfigDir}\"",
					$"-o \"{base.GetConfiguration().SummaryReportPath}\"",
					$"-metadatafilter \"{string.Join(" and ", Filter)}\"",
					$"{ReportTypeCmdLineParam}",
					"-summaryTable autoPerfReportStandard",
					"-condensedSummaryTable autoPerfReportStandard",
					"-emailtable",
					"-recurse"
				};

				var ArgStr = string.Join(" ", Args);

				CommandUtils.RunAndLog(ToolPath.FullName, ArgStr, out ErrorCode);
				if (ErrorCode != 0)
				{
					Logger.LogError($"PerfReportTool returned error code \"{ErrorCode}\" while generating HTML for the historic report.");
				}


				// Generate CSV report
				Args = new[]
				{
					$"-summarytablecachein \"{ReportCacheDir}\"",
					$"-reportxmlbasedir \"{ReportConfigDir}\"",
					$"-o \"{base.GetConfiguration().SummaryReportPath}\"",
					$"-metadatafilter \"{string.Join(" and ", Filter)}\"",
					$"-summaryTableFilename \"{Name  + GetTestSuffix()}\"",
					$"{ReportTypeCmdLineParam}",
					"-csvTable",
					"-summaryTable autoPerfReportStandard",
					"-condensedSummaryTable autoPerfReportStandard",
					"-emailtable",
					"-recurse"
				};

				ArgStr = string.Join(" ", Args);

				CommandUtils.RunAndLog(ToolPath.FullName, ArgStr, out ErrorCode);
				if (ErrorCode != 0)
				{
					Logger.LogError($"PerfReportTool returned error code \"{ErrorCode}\" while generating CSV for the historic report.");
				}
			}

			// Generates HTML report to Reports/SaloonPerf/SaloonWin64SaloonPerf/ICVFXTest.PerformanceReport_PLATFORM/Reports/Performance
			void HistoricReport_Alt(string Name, IEnumerable<string> Filter)
			{	
				var Args = new[]
				{
					$"-summarytablecachein \"{ReportCacheDir}\"",
					$"-summaryTableFilename \"{Name  + GetTestSuffix()}.html\"",
					$"-reportxmlbasedir \"{ReportConfigDir}\"",
					$"-o \"{ReportPath}\"",
					$"-metadatafilter \"{string.Join(" and ", Filter)}\"",
					$"{ReportTypeCmdLineParam}",
					"-summaryTable autoPerfReportStandard",
					"-condensedSummaryTable autoPerfReportStandard",
					"-emailtable",
					"-recurse"
				};

				var ArgStr = string.Join(" ", Args);

				CommandUtils.RunAndLog(ToolPath.FullName, ArgStr, out ErrorCode);
				if (ErrorCode != 0)
				{
					Logger.LogError($"PerfReportTool returned error code \"{ErrorCode}\" while generating historic report.");
				}
			}

			// Creates/Updates all-time historic summary report
			HistoricReport_Alt("HistoricReport_AllTime", new[]
			{
				$"platform={PlatformNameFilter}"
			});

			// Creates/Updates 14 days historic report
			HistoricReport_Alt($"HistoricReport_14Days", new[]
			{
				$"platform={PlatformNameFilter}",
				$"starttimestamp>={DateTimeOffset.Now.ToUnixTimeSeconds() - (14 * 60L * 60L * 24L)}"
			});

			// Creates/Updates 14 days historic report
			HistoricReport($"HistoricReport_14Days_Summary", new[]
			{
				$"platform={PlatformNameFilter}",
				$"starttimestamp>={DateTimeOffset.Now.ToUnixTimeSeconds() - (14 * 60L * 60L * 24L)}"
			});
		}

		public override bool StartTest(int Pass, int InNumPasses)
		{
			// Delete temporary folder for CSVs when rerunning
			if (Pass == 0 && TempPerfCSVDir.Exists)
			{
				TempPerfCSVDir.Delete(recursive: true);
			}
			return base.StartTest(Pass, InNumPasses);
		}

		public override ITestReport CreateReport(TestResult Result, UnrealTestContext Context, UnrealBuildSource Build, IEnumerable<UnrealRoleResult> Artifacts, string ArtifactPath)
		{
			if (Result == TestResult.Passed)
			{
				ICVFXTestConfig Config = base.GetConfiguration();

				// Temporarily copy artifacts
				CopyPerfFilesToTempDir(Context.GetRoleContext(UnrealTargetRole.Client).Platform, ArtifactPath, Context.Options.TempDir);

				// Preserve artifacts till ready to be made into a report on the final iteration
				if (GetCurrentPass() < (GetNumPasses() - 1))
				{
					Logger.LogInformation($"Skipping CSV report generator until final pass. On pass {GetCurrentPass() + 1} of {GetNumPasses()}.");
					return base.CreateReport(Result, Context, Build, Artifacts, ArtifactPath);
				}

				// Create a local report
				if (!Config.NoLocalReports)
				{
					Logger.LogInformation($"Generating performance reports using PerfReportTool.");
					GeneratePerfReport(Context.GetRoleContext(UnrealTargetRole.Client).Platform, ArtifactPath, Context.Options.TempDir);
				}

				// Build import entries for PRS
				if (!Config.SkipPerfReportServer)
				{
					Logger.LogInformation("Creating perf server importer with build name {BuildName}", OriginalBuildName);

					string DataSourceName = "Automation.Saloon.ICVFXTesting";

					string ImportDirOverride;
					if(Config.PerfReportServerImportDir == "") {
						ImportDirOverride = null;
					} else {
						ImportDirOverride = Config.PerfReportServerImportDir;
					}
					
					Dictionary<string, dynamic> CommonDataSourceFields = new Dictionary<string, dynamic>
					{
						{ "HordeJobUrl", Globals.Params.ParseValue("JobDetails", null) }
					};

					ICsvImporter Importer = ReportGenUtils.CreatePerfReportServerImporter(DataSourceName, OriginalBuildName, CommandUtils.IsBuildMachine, ImportDirOverride, CommonDataSourceFields);
					if (Importer != null)
					{
						// Recursively grab all the csv files we copied to the temp dir and convert them to binary.
						List<FileInfo> AllBinaryCsvFiles = ReportGenUtils.CollectAndConvertCsvFilesToBinary(TempPerfCSVDir.FullName);
						if (AllBinaryCsvFiles.Count == 0)
						{
							throw new AutomationException($"No CSV files found in {TempPerfCSVDir}");
						}

						// The corresponding log for each csv sits in the same subdirectory as the csv file itself.
						IEnumerable<CsvImportEntry> ImportEntries = AllBinaryCsvFiles
							.Select(CsvFile => new CsvImportEntry(CsvFile.FullName, Path.Combine(CsvFile.Directory.FullName, "ClientOutput.log")));

						// Create the import batch
						Logger.LogInformation("Importing entries to {DataSourceName}", DataSourceName);
						Importer.Import(ImportEntries);
					}

					// Cleanup the temp dir
					TempPerfCSVDir.Delete(recursive: true);
				}
			}
			else
			{
				Logger.LogWarning($"Skipping performance report generation because the perf report test failed.");
			}

			return base.CreateReport(Result, Context, Build, Artifacts, ArtifactPath);
		}

		// Copy CSV files to temp directory because of duplicate import batches
		private void CopyPerfFilesToTempDir(UnrealTargetPlatform Platform, string ArtifactPath, string TempDir)
		{
			if (!TempPerfCSVDir.Exists)
			{
				Logger.LogInformation("Creating temp perf CSV dir: {TempPerfCSVDir}", TempPerfCSVDir);
				TempPerfCSVDir.Create();
			}

			string TestInstancePath = Path.Combine(TempDir, "DeviceCache", Platform.ToString(), TestInstance.ClientApps[0].Device.ToString());
			string ClientLogPath = Path.Combine(TestInstancePath, "UserDir", "Saved", "Logs", "Saloon.log");
			string FPSChartsPath = GetNewestCSVDir(Platform, ArtifactPath, TempDir);
			
			if (string.IsNullOrEmpty(FPSChartsPath))
			{
				Logger.LogWarning("Failed to find FPSCharts folder in {ArtifactPath}", ArtifactPath);
				return;
			}

			FPSChartsPath = Gauntlet.Utils.SystemHelpers.GetFullyQualifiedPath(FPSChartsPath);
			
			Logger.LogInformation($"Using client log path \"{ClientLogPath}\".");
			Logger.LogInformation($"Using FPS charts path \"{FPSChartsPath}\".");

			// Grab all the csv files that have valid metadata
			List<FileInfo> CsvFiles = ReportGenUtils.CollectValidCsvFiles(FPSChartsPath);
			if (CsvFiles.Count > 0)
			{
				// We only want to copy the latest file as the other will have already been copied
				CsvFiles.OrderBy(Info => Info.LastWriteTimeUtc);
				FileInfo LatestCsvFile = CsvFiles.Last();

				// Create a subdir for each pass as we want to store the csv and log together in the same dir to make it easier to find them
				string PassDir = Path.Combine(TempPerfCSVDir.FullName, $"PerfCsv_Pass_{GetCurrentPass()}");
				Directory.CreateDirectory(PassDir);

				FileInfo LogFileInfo = new FileInfo(ClientLogPath);
				if (LogFileInfo.Exists)
				{
					string LogDestPath = Path.Combine(PassDir, LogFileInfo.Name);
					Logger.LogInformation("Copying Log {ClientLogPath} To {LogDest}", ClientLogPath, LogDestPath);
					LogFileInfo.CopyTo(LogDestPath);
				}
				else
				{
					Logger.LogWarning("No log file was found at {ClientLogPath}", ClientLogPath);
				}

				string CsvDestPath = Path.Combine(PassDir, LatestCsvFile.Name);
				Logger.LogInformation("Copying CSV {CsvPath} To {CsvDestPath}", LatestCsvFile.FullName, CsvDestPath);
				LatestCsvFile.CopyTo(CsvDestPath);
			}
			else
			{
				Logger.LogWarning("No valid CSV files found in {FPSChartsPath}", FPSChartsPath);
			}
		}
	}
  
	//
	// Horrible hack to repeat the perf tests 3 times...
	// There is no way to pass "-repeat=N" to Gauntlet via the standard engine build scripts, nor is
	// it possible to override the number of iterations per-test via the GetConfiguration() function.
	//
	// In theory we can pass the "ICVFXTest.PerformanceReport" test name to Gauntlet 3 times via Horde scripts,
	// but the standard build scripts will attempt to define 3 nodes all with the same name, which won't work.
	//
	// These three classes allow us to run 3 copies of the PerformanceReport test, but ensures they all have 
	// different names to fit into the build script / Gauntlet structure.
	//

	public class PerformanceReport_1 : PerformanceReport
	{
		public PerformanceReport_1(UnrealTestContext InContext) : base(InContext) { }
	}

	public class PerformanceReport_2 : PerformanceReport
	{
		public PerformanceReport_2(UnrealTestContext InContext) : base(InContext) { }
	}

	public class PerformanceReport_3 : PerformanceReport
	{
		public PerformanceReport_3(UnrealTestContext InContext) : base(InContext) { }
	}

	public class PerformanceReport_MGPU : PerformanceReport
	{
		public PerformanceReport_MGPU(UnrealTestContext InContext) : base(InContext)
		{
	
		}
		public override int GetMaxGPUCount() 
		{
			return 2;
		}

		public override string GetTestSuffix()
		{
			return "MGPU";
		}
	}
	public class PerformanceReport_NaniteLumen : PerformanceReport
	{
		public PerformanceReport_NaniteLumen(UnrealTestContext InContext) : base(InContext)
		{

		}
		public override bool IsLumenEnabled()
		{
			return true;
		}
		public override bool UseNanite()
		{
			return true;
		}

		public override string GetTestSuffix()
		{
			return "NaniteLumen";
		}
	}

	public class PerformanceReport_Vulkan : PerformanceReport
	{
		public PerformanceReport_Vulkan(UnrealTestContext InContext) : base(InContext)
		{

		}

		public override string GetTestSuffix()
		{
			return "Vulkan";
		}

		public override bool UseVulkan()
		{
			return true;
		}
	}

	public class PerformanceReport_Nanite : PerformanceReport
	{
		public PerformanceReport_Nanite(UnrealTestContext InContext) : base(InContext)
		{

		}

		public override string GetTestSuffix()
		{
			return "Nanite";
		}

		public override bool UseNanite()
		{
			return true;
		}
	}
}
