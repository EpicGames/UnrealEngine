<Project Sdk="Microsoft.NET.Sdk">
  <Import Project="ICVFX.Automation.csproj.props" Condition="Exists('ICVFX.Automation.csproj.props')" />
  <Import Project="..\..\..\..\..\Source\Programs\Shared\UnrealEngine.csproj.props" />
  <PropertyGroup>
    <OutputType>Library</OutputType>
    <GenerateAssemblyInfo>false</GenerateAssemblyInfo>
    <GenerateTargetFrameworkAttribute>false</GenerateTargetFrameworkAttribute>
    <RootNamespace>ICVFX.Automation</RootNamespace>
    <AssemblyName>ICVFX.Automation</AssemblyName>
    <WarningsNotAsErrors>612,618</WarningsNotAsErrors>
    <OutputPath Condition="'$(ArtifactsPath)' == ''">..\..\..\..\..\Binaries\DotNET\AutomationTool\AutomationScripts\ICVFX</OutputPath>
    <TreatWarningsAsErrors>true</TreatWarningsAsErrors>
  </PropertyGroup>
  <ItemGroup>
    <Compile Include="$(EngineDir)\Source\Programs\Shared\MetaData.cs">
      <Link>Properties\MetaData.cs</Link>
    </Compile>
  </ItemGroup>
  <ItemGroup>
    <PackageReference Include="System.Data.DataSetExtensions" Version="4.5.0" />
  </ItemGroup>
  <ItemGroup>
    <ProjectReference Include="$(EngineDir)\Source\Programs\AutomationTool\AutomationUtils\AutomationUtils.Automation.csproj" />
    <ProjectReference Include="$(EngineDir)\Source\Programs\AutomationTool\Gauntlet\Gauntlet.Automation.csproj" />
    <ProjectReference Include="$(EngineDir)\Source\Programs\AutomationTool\Scripts\AutomationScripts.Automation.csproj" />
    <ProjectReference Include="$(EngineDir)\Source\Programs\AutomationTool\Localization\Localization.Automation.csproj" />
    <ProjectReference Include="$(EngineDir)\Source\Programs\AutomationTool\XLocLocalization\XLocLocalization.Automation.csproj" />
    <ProjectReference Include="$(EngineDir)\Source\Programs\Shared\EpicGames.Core\EpicGames.Core.csproj" />
    <ProjectReference Include="$(EngineDir)\Source\Programs\Shared\EpicGames.Build\EpicGames.Build.csproj" />
    <ProjectReference Include="$(EngineDir)\Source\Programs\UnrealBuildTool\UnrealBuildTool.csproj" />
  </ItemGroup>
</Project>
