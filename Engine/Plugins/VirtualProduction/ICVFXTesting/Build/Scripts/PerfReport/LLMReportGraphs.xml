<?xml version="1.0" encoding="UTF-8"?>

<!-- This structure describes how to graph out the different graphs specified in ReportTypes.xml
       The base settings are the main settings that all of the other graphs use unless they specifically override them.
       The main base settings are:
         - maxy:           This is the maximum value that the graph will represent, so make sure they covers the whole dataset.
         - stacked:        Indicates whether the output graph will be stacked.
         - hideStatPrefix: This indicates the prefix that should be hidden in the graph legend.
		 - showEvents:     Semicolon delimited list of event names to show. (can include wildcards)
         
       The graph node can take the same attributes as the baseSettings node (if it does it will override them, otherwise, it will use the baseSettings settings.
       The graph has a title attribute, this is the title of the graph, this MUST match the appropriate graph node title in ReportTypes.xml.
       The graph also has a subnode called statString, this is the string that is passed to the CSVToSVG tool, therefore * will graph all of the stats in the CSV,
       while something like "name1 name2" will graph only the two stats with headings name1 and name2.
       
  -->

<graphGroups>
	<baseSettings thickness="1" smooth="0" smoothKernelSize="-1" smoothKernelPercent="1" compression="0.1"
				  width="1800" height="600" stacked="0" maxHierarchyDepth="2" hideStatPrefix="" showEvents="MapSetup*;Frontend*;Match*;Wave*"/>

	<!-- a graph group is a convenient way to apply common settings to multiple graphs, avoiding duplication -->
	<graphGroup name="LLMBase">
		<baseSettings maxHierarchyDepth="4" stacked="0" hideStatPrefix="Meshes/;Audio/;" requiresDetailedStats="0" thickness="100" compression="0.05" showEvents="*" />

		<graph title="Total">
			<statString>Total Untracked TrackedTotal</statString>
		</graph>

		<!-- ignoreStats should include all parent LLM tags -->
		<graph title="Memory Allocated" stacked="1" thickness="100" maxHierarchyDepth="1" ignoreStats="TrackedTotal;Audio;Meshes;Physics" mainStat="Total">
			<statString>*</statString>
		</graph>

		<!-- top level LLM summary graphs -->
		<graph title="Content Summary" stacked="1" thickness="100" maxHierarchyDepth="-1">
			<statString>Shaders Materials MaterialInstance Meshes/* Audio/AudioSoundWaves Animation</statString>
		</graph>
		<graph title="Misc Summary">
			<statString>EngineInit EnginePreInit EngineMisc TaskGraphMiscTasks FName GC Localization AssetRegistry ConfigSystem RealTimeCommunications</statString>
		</graph>
		<graph title="Rendering Summary">
			<statString>RenderingThread RenderTargets PSO SceneRender RHIMisc Graphics</statString>
		</graph>
		<graph title="Development Summary">
			<statString>Stats CsvProfiler</statString>
		</graph>	

		<!-- general graphs -->
		<graph title="Tracked Total" maxy="8000">
			<statString>TrackedTotal</statString>
		</graph>
		<graph title="Untagged">
			<statString>Untagged</statString>
		</graph>
		<graph title="Untracked">
			<statString>Untracked</statString>
		</graph>
		<graph title="ProgramSize">
			<statString>ProgramSize</statString>
		</graph>
		<graph title="EnginePreInitMemory">
			<statString>EnginePreInit</statString>
		</graph>
		<graph title="Stats">
			<statString>Stats</statString>
		</graph>
		<graph title="UObject">
			<statString>UObject</statString>
		</graph>
		<graph title="RHI Misc">
			<statString>RHIMisc</statString>
		</graph>
		<graph title="Shaders">
			<statString>Shaders</statString>
		</graph>
		<graph title="Render Targets">
			<statString>RenderTargets</statString>
		</graph>
		<graph title="Textures">
			<statString>Textures</statString>
		</graph>
		<graph title="AsyncLoading">
			<statString>AsyncLoading</statString>
		</graph>
		<graph title="Materials">
			<statString>Materials</statString>
		</graph>
		<graph title="Audio">
			<statString>Audio</statString>
		</graph>
		<graph title="Audio Detail">
			<statString>Audio/*</statString>
		</graph>
		<graph title="Meshes Detail">
			<statString>Meshes/*</statString>
		</graph>
		<graph title="EngineInitMemory">
			<statString>EngineInit</statString>
		</graph>
		<graph title="Animation">
			<statString>Animation</statString>
		</graph>
		<graph title="RenderingThreadMemory">
			<statString>RenderingThread</statString>
		</graph>
		<graph title="LoadMapMemory">
			<statString>LoadMapMisc</statString>
		</graph>
		<graph title="Physics">
			<statString>Physics</statString>
		</graph>
		<graph title="Physics Detail">
			<statString>Physics/*</statString>
		</graph>
		<graph title="EngineMisc">
			<statString>EngineMisc</statString>
		</graph>
		<graph title="StreamingManager">
			<statString>StreamingManager</statString>
		</graph>
	</graphGroup>

	<graphGroup name="LLMPlatformBase">
		<baseSettings maxHierarchyDepth="4" maxy="1024" stacked="0" hideStatPrefix="" requiresDetailedStats="0" thickness="100" compression="0.05" showEvents="*" />

		<graph title="Platform FMalloc" maxy="2500">
			<statString>FMalloc</statString>
		</graph>
		<graph title="Platform GenericPlatformMallocCrash" maxy="5">
			<statString>GenericPlatformMallocCrash</statString>
		</graph>
		<graph title="Platform Graphics" maxy="2000">
			<statString>Graphics</statString>
		</graph>
		<graph title="Platform Thread Stack" maxy="10">
			<statString>ThreadStack</statString>
		</graph>
	</graphGroup>
</graphGroups>
