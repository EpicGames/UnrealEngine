<?xml version="1.0" encoding="UTF-8"?>

<!-- The report type works by having a summary type and different graphs.
       The summary type must match one of the types in PerfReportTool, currently we only have:
         - Peak
         - Histogram
       
       To create a new type just subclass Summary and override its functions. See PeakSummary or HistogramSummary as examples.
       
       Then each graph has a title, budget, inSummary, inMainSummary and ignoreList.
        - title:         Used to link the graph to the graph in ReportGraphs.xml, so they must be the same.
        - budget:        This is the value that we are going to compare to. How much we can use.
        - inSummary:     This indicated whether this should be included in the HTML report produced.
        - inMainSummary: This indicated whether this is in the main report at the top.
        - ignoreList:    This is mainly used with wildcards, if you have stats in the CSV you want ignored for this graph, add them to this list.
                         It's a semi-colon separated list as such: "name1;name2;..."
       
       NOTE: Each graph needs an entry in the appropriate graphGroup node in ReportGraphs.xml, otherwise the program crashes.
  -->
<root>

	<summaryTables>
		<summaryTable name="default" rowSort="CSV File">
			<filter>Total Max,Untracked Avg,CSV File,Report,frameCount,</filter>
		</summaryTable>

	</summaryTables>

	<reporttypes reportGraphsFile="LLMReportGraphs.xml">

		<!-- LLM Report -->
		<reporttype name="LLM" title="LLM Report" vsync="0" ignoreList="">

			<autodetection requiredstats="EngineInit">
			</autodetection>

			<summary type="peak">
				<colourThresholds stat="Total">3000,4000,5000,6000</colourThresholds>
			</summary>

			<!-- top-level LLM summary graphs -->
			<graph title="Total"                       budget="4540"    inSummary="1"  inMainSummary="1"/>
			<graph title="Content Summary"             budget="1500"    inSummary="0"/>
			<graph title="Rendering Summary"           budget="250"     inSummary="0"/>
			<graph title="Misc Summary"                budget="75"      inSummary="0"/>
			<graph title="Development Summary"         budget="35"      inSummary="0"/>
			
			<graph title="Tracked Total"               budget="2570"    inSummary="0"/>
			<!-- Sum of below -->
			<graph title="Animation"                   budget="23"      inSummary="1"/>
			<graph title="AsyncLoading"                budget="36"      inSummary="1"/>
			<graph title="Audio"                       budget="96"      inSummary="1"/>
			<graph title="Audio Detail"                budget="96"      inSummary="0"/>
			<graph title="Meshes Detail"               budget="0"       inSummary="0"/>
			<graph title="EngineInitMemory"            budget="6"       inSummary="1"/>
			<graph title="EnginePreInitMemory"         budget="167.69"  inSummary="1"/>
			<graph title="EngineMisc"                  budget="104.43"  inSummary="1"/>
		<!--<graph title="FileSystem"                  budget="64"      inSummary="1"/> -->
			<graph title="LoadMapMemory"               budget="18"      inSummary="1"/>
		<!--<graph title="Localization"                budget="64"      inSummary="1"/> -->
			<graph title="Materials"                   budget="7"       inSummary="1"/>
		<!--<graph title="MediaStreaming"              budget="20"      inSummary="0"/> -->
			<graph title="Physics"                     budget="45"      inSummary="1"/>
			<graph title="Physics Detail"              budget="45"      inSummary="0"/>
			<graph title="ProgramSize"                 budget="233"     inSummary="1"/>
			<graph title="Render Targets"              budget="101"     inSummary="1"/>
			<graph title="RenderingThreadMemory"       budget="170"     inSummary="1"/>
			<graph title="RHI Misc"                    budget="67"      inSummary="1"/>
			<graph title="Shaders"                     budget="271"     inSummary="1"/>
			<graph title="Stats"                       budget="3"       inSummary="1"/>
			<graph title="StreamingManager"            budget="6"       inSummary="1"/>
			<graph title="Textures"                    budget="304"     inSummary="1"/>
			<graph title="UObject"                     budget="343"     inSummary="1"/>
			<graph title="Untagged"                    budget="26.21"   inSummary="1"/>
			
			<graph title="Memory Allocated"            budget="0"       inSummary="0"/>
		</reporttype>

		<!-- LLM Report -->
		<reporttype name="LLMSwitch" title="LLM Switch Base Report" vsync="0" ignoreList="">

			<summary type="peak">
				<colourThresholds stat="Total">3000,4000,5000,6000</colourThresholds>
			</summary>

			<!-- top-level LLM summary graphs -->
			<graph title="Total"                       budget="3103"    inSummary="1" inMainSummary="1"/>
			<graph title="Content Summary"             budget="1500"    inSummary="0"/>
			<graph title="Rendering Summary"           budget="250"     inSummary="0"/>
			<graph title="Misc Summary"                budget="75"      inSummary="0"/>
			<graph title="Development Summary"         budget="35"      inSummary="0"/>

			<graph title="Tracked Total"               budget="2570"    inSummary="0"/>
			<!-- Sum of below -->
			<graph title="Animation"                   budget="23"      inSummary="1"/>
			<graph title="AsyncLoading"                budget="36"      inSummary="1"/>
			<graph title="Audio"                       budget="96"      inSummary="1"/>
			<graph title="Audio Detail"                budget="96"      inSummary="0"/>
			<graph title="Meshes Detail"               budget="0"       inSummary="0"/>
			<graph title="EngineInitMemory"            budget="6"       inSummary="1"/>
			<graph title="EnginePreInitMemory"         budget="167.69"  inSummary="1"/>
			<graph title="EngineMisc"                  budget="104.43"  inSummary="1"/>
			<graph title="FileSystem"                  budget="64"      inSummary="1"/>
			<graph title="LoadMapMemory"               budget="18"      inSummary="1"/>
			<graph title="Localization"                budget="64"      inSummary="1"/>
			<graph title="Materials"                   budget="7"       inSummary="1"/>
			<graph title="MediaStreaming"              budget="20"      inSummary="0"/>
			<graph title="Physics"                     budget="45"      inSummary="1"/>
			<graph title="Physics Detail"              budget="45"      inSummary="0"/>
			<graph title="ProgramSize"                 budget="233"     inSummary="1"/>
			<graph title="Render Targets"              budget="101"     inSummary="1"/>
			<graph title="RenderingThreadMemory"       budget="170"     inSummary="1"/>
			<graph title="RHI Misc"                    budget="67"      inSummary="1"/>
			<graph title="Shaders"                     budget="271"     inSummary="1"/>
			<graph title="Stats"                       budget="3"       inSummary="1"/>
			<graph title="StreamingManager"            budget="6"       inSummary="1"/>
			<graph title="Textures"                    budget="304"     inSummary="1"/>
			<graph title="Untagged"                    budget="123"     inSummary="1"/>
			<graph title="UObject"                     budget="343"     inSummary="1"/>
			
			<graph title="Memory Allocated"            budget="4540"    inSummary="0"/>
		</reporttype>

		<!-- LLM Report -->
		<reporttype name="LLMXboxOne" title="LLM XboxOne Base Report" vsync="0" ignoreList="">

			<summary type="peak">
				<colourThresholds stat="Total">3000,4000,5000,6000</colourThresholds>
			</summary>

			<!-- top-level LLM summary graphs -->
			<graph title="Total"                       budget="4491"    inSummary="1" inMainSummary="1"/>
			<graph title="Content Summary"             budget="1500"    inSummary="0"/>
			<graph title="Rendering Summary"           budget="250"     inSummary="0"/>
			<graph title="Misc Summary"                budget="75"      inSummary="0"/>
			<graph title="Development Summary"         budget="35"      inSummary="0"/>

			<graph title="Tracked Total"               budget="4315"    inSummary="0"/>
			<!-- Sum of below -->
			<graph title="Animation"                   budget="32"      inSummary="1"/>
			<graph title="AsyncLoading"                budget="51"      inSummary="1"/>
			<graph title="Audio"                       budget="301"     inSummary="1"/>
			<graph title="Audio Detail"                budget="301"     inSummary="0"/>
			<graph title="Meshes Detail"               budget="0"       inSummary="0"/>
			<graph title="EngineInitMemory"            budget="17"      inSummary="1"/>
			<graph title="EnginePreInitMemory"         budget="167.69"  inSummary="1"/>
			<graph title="EngineMisc"                  budget="104.43"  inSummary="1"/>
			<graph title="FileSystem"                  budget="64"      inSummary="1"/>
			<graph title="LoadMapMemory"               budget="21"      inSummary="1"/>
			<graph title="Localization"                budget="64"      inSummary="1"/>
			<graph title="Materials"                   budget="8"       inSummary="1"/>
			<graph title="MediaStreaming"              budget="20"      inSummary="0"/>
			<graph title="Physics"                     budget="45"      inSummary="1"/>
			<graph title="Physics Detail"              budget="45"      inSummary="0"/>
			<graph title="ProgramSize"                 budget="274"     inSummary="1"/>
			<graph title="Render Targets"              budget="158"     inSummary="1"/>
			<graph title="RenderingThreadMemory"       budget="73"      inSummary="1"/>
			<graph title="RHI Misc"                    budget="90"      inSummary="1"/>
			<graph title="Shaders"                     budget="342"     inSummary="1"/>
			<graph title="Stats"                       budget="4"       inSummary="1"/>
			<graph title="StreamingManager"            budget="10"      inSummary="1"/>
			<graph title="Textures"                    budget="947"     inSummary="1"/>
			<graph title="Untagged"                    budget="77"      inSummary="1"/>
			<graph title="UObject"                     budget="534"     inSummary="1"/>
			
			<graph title="Memory Allocated"            budget="4540"    inSummary="0"/>
		</reporttype>

		<!-- LLM Report -->
		<reporttype name="LLMPS4" title="LLM PS4 Base Report" vsync="0" ignoreList="">

			<summary type="peak">
				<colourThresholds stat="Total">3000,4000,5000,6000</colourThresholds>
			</summary>

			<!-- top-level LLM summary graphs -->
			<graph title="Total"                       budget="4113"    inSummary="1" inMainSummary="1"/>
			<graph title="Content Summary"             budget="1500"    inSummary="0"/>
			<graph title="Rendering Summary"           budget="250"     inSummary="0"/>
			<graph title="Misc Summary"                budget="75"      inSummary="0"/>
			<graph title="Development Summary"         budget="35"      inSummary="0"/>

			<graph title="Tracked Total"               budget="3776"    inSummary="0"/>
			<!-- Sum of below -->
			<graph title="Animation"                   budget="37"      inSummary="1"/>
			<graph title="AsyncLoading"                budget="51"      inSummary="1"/>
			<graph title="Audio"                       budget="118"     inSummary="1"/>
			<graph title="Audio Detail"                budget="118"     inSummary="1"/>
			<graph title="Meshes Detail"               budget="0"       inSummary="0"/>
			<graph title="EngineInitMemory"            budget="18"      inSummary="1"/>
			<graph title="EnginePreInitMemory"         budget="167.69"  inSummary="1"/>
			<graph title="EngineMisc"                  budget="104.43"  inSummary="1"/>
			<graph title="FileSystem"                  budget="64"      inSummary="1"/>
			<graph title="LoadMapMemory"               budget="23"      inSummary="1"/>
			<graph title="Localization"                budget="64"      inSummary="1"/>
			<graph title="Materials"                   budget="8"       inSummary="1"/>
			<graph title="MediaStreaming"              budget="20"      inSummary="0"/>
			<graph title="Physics"                     budget="44"      inSummary="1"/>
			<graph title="Physics Detail"              budget="44"      inSummary="0"/>
			<graph title="ProgramSize"                 budget="207"     inSummary="1"/>
			<graph title="Render Targets"              budget="282"     inSummary="1"/>
			<graph title="RenderingThreadMemory"       budget="55"      inSummary="1"/>
			<graph title="RHI Misc"                    budget="25"      inSummary="1"/>
			<graph title="Shaders"                     budget="193"     inSummary="1"/>
			<graph title="Stats"                       budget="4"       inSummary="1"/>
			<graph title="StreamingManager"            budget="6"       inSummary="1"/>
			<graph title="Textures"                    budget="782"     inSummary="1"/>
			<graph title="Untagged"                    budget="72"      inSummary="1"/>
			<graph title="UObject"                     budget="732"     inSummary="1"/>
			
			<graph title="Memory Allocated"            budget="4540"    inSummary="0"/>
		</reporttype>

		<!-- LLM Report -->
		<reporttype name="LLMWin64" title="LLM Win64 Base Report" vsync="0" ignoreList="">

			<summary type="peak">
				<colourThresholds stat="Total">3000,4000,5000,6000</colourThresholds>
			</summary>

			<!-- top-level LLM summary graphs -->
			<graph title="Total"                       budget="4540"    inSummary="1" inMainSummary="1"/>
			<graph title="Content Summary"             budget="1500"    inSummary="0"/>
			<graph title="Development Summary"         budget="35"      inSummary="0"/>
			<graph title="Rendering Summary"           budget="250"     inSummary="0"/>
			<graph title="Misc Summary"                budget="75"      inSummary="0"/>

			<graph title="Tracked Total"               budget="2790"    inSummary="0"/>
			<!-- Sum of below -->
			<graph title="Animation"                   budget="37"      inSummary="1"/>
			<graph title="AsyncLoading"                budget="55"      inSummary="1"/>
			<graph title="Audio"                       budget="86"      inSummary="1"/>
			<graph title="Audio Detail"                budget="86"      inSummary="1"/>
			<graph title="Meshes Detail"               budget="0"       inSummary="0"/>
			<graph title="EngineInitMemory"            budget="18"      inSummary="1"/>
			<graph title="EnginePreInitMemory"         budget="167.69"  inSummary="1"/>
			<graph title="EngineMisc"                  budget="104.43"  inSummary="1"/>
			<graph title="FileSystem"                  budget="64"      inSummary="1"/>
			<graph title="LoadMapMemory"               budget="20"      inSummary="1"/>
			<graph title="Localization"                budget="64"      inSummary="1"/>
			<graph title="Materials"                   budget="7"       inSummary="1"/>
			<graph title="MediaStreaming"              budget="20"      inSummary="0"/>
			<graph title="Physics"                     budget="46"      inSummary="1"/>
			<graph title="Physics Detail"              budget="46"      inSummary="0"/>
			<graph title="ProgramSize"                 budget="189"     inSummary="1"/>
			<graph title="Render Targets"              budget="40"      inSummary="1"/>
			<graph title="RenderingThreadMemory"       budget="48"      inSummary="1"/>
			<graph title="RHI Misc"                    budget="5"       inSummary="1"/>
			<graph title="Shaders"                     budget="199"     inSummary="1"/>
			<graph title="Stats"                       budget="5"       inSummary="1"/>
			<graph title="StreamingManager"            budget="5"       inSummary="1"/>
			<graph title="Textures"                    budget="445"     inSummary="1"/>
			<graph title="Untagged"                    budget="96"      inSummary="1"/>
			<graph title="UObject"                     budget="655"     inSummary="1"/>

			<graph title="Memory Allocated"            budget="4540"    inSummary="0"/>
		</reporttype>

		<!-- LLM Report -->
		<reporttype name="LLMPlatformBase" title="LLM Platform Base Report" vsync="0" ignoreList="">
			<autodetection requiredstats="LLMOverhead">
			</autodetection>
			
			<summary type="peak">
				<colourThresholds stat="Total">3000,4000,5000,6000</colourThresholds>
			</summary>

			<graph title="Total"                                budget="4540"     inSummary="1" inMainSummary="1"/>
			<graph title="Memory Allocated"                     budget="4540"     inSummary="0"/>
			<graph title="Untagged"                             budget="0"        inSummary="1"/>
			<graph title="Platform Graphics"                    budget="1500.0"   inSummary="1"/>
			<graph title="ProgramSize"                          budget="275.0"    inSummary="1"/>
			<graph title="Platform FMalloc"                     budget="2200"     inSummary="1"/>
			<graph title="Platform GenericPlatformMallocCrash"  budget="4.0"      inSummary="1"/>
			<graph title="Platform Thread Stack"                budget="7.5"      inSummary="1"/>

		</reporttype>

	</reporttypes>
</root>