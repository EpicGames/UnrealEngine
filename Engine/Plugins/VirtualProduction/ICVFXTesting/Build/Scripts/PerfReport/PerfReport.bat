@echo off
call %~dp0\SetupEnv.bat

if "%~1"=="" (
  %CSVToolsPath%\PerfReportTool
  goto exit
)

if "%~2"=="" (
  @echo on
  %CSVToolsPath%\PerfReportTool -reportxmlbasedir %~dp0 -csv %1
  goto exit
) 

%CSVToolsPath%\PerfReportTool -reportxmlbasedir %~dp0 %* 

:exit
