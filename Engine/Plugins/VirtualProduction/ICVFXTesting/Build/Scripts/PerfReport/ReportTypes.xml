<?xml version="1.0" encoding="UTF-8"?>

<root>
	<!-- Friendly names for display purposes -->
	<statDisplayNameMappings>
		<mapping statName="RHI/Drawcalls" displayName="DrawCalls"/>
		<mapping statName="DynamicResolutionPercentage" displayName="DynRes %"/>
		<mapping statName="Basic/TicksQueued" displayName="TicksQueued"/>
		<mapping statName="Ticks/PlayerPawns" displayName="PlayerPawns"/>
        <mapping statName="Ticks/Vehicles" displayName="Vehicles"/>
		<mapping statName="MemoryFreeMB" displayName="MemFreeMB"/>
		<mapping statName="PhysicalUsedMB" displayName="MemUsedMB"/>
		<mapping statName="buildversion" displayName="Build Version"/>
		<mapping statName="testbuildversion" displayName="Build Info"/>
		<mapping statName="config" displayName="Config"/>
		<mapping statName="commandline" displayName="Commandline"/>
		<mapping statName="syncLoads" displayName="Sync Loads"/>
		<mapping statName="scalabilitylevel" displayName="Scaling"/>
		<mapping statName="deviceprofile" displayName="DeviceProfile"/>
		<mapping statName="TargetFPS" displayName="FPS"/>
		<mapping statName="HitchTimePercent" displayName="HitchTime %"/>
		<mapping statName="testname" displayName="Test Name"/>
		<mapping statName="platform" displayName="Platform"/>
		<mapping statName="csvfilename" displayName="CSV Filename"/>
		<mapping statName="rhiname" displayName="RHI Name"/>
		<mapping statName="desktophwinfo" displayName="Hardware"/>
		<mapping statName="fmsglogf/fmsglogfcount" displayName="Logs/frame"/>
	</statDisplayNameMappings>

	<derivedMetadataMappings>
		<mapping destName="deviceType" destValue="console" sourceName="platform" sourceValue="ps4"/>
		<mapping destName="deviceType" destValue="console" sourceName="platform" sourceValue="ps5"/>
		<mapping destName="deviceType" destValue="console" sourceName="platform" sourceValue="switch"/>
		<mapping destName="deviceType" destValue="console" sourceName="platform" sourceValue="xb1"/>
		<mapping destName="deviceType" destValue="console" sourceName="platform" sourceValue="xboxone"/>
		<mapping destName="deviceType" destValue="console" sourceName="platform" sourceValue="xboxonegdk"/>
		<mapping destName="deviceType" destValue="console" sourceName="platform" sourceValue="xsx"/>
		<mapping destName="devicetype" destValue="desktop" sourceName="platform" sourceValue="mac"/>
		<mapping destName="devicetype" destValue="desktop" sourceName="platform" sourceValue="windows"/>
		<mapping destName="devicetype" destValue="mobile" sourceName="platform" sourceValue="android"/>
		<mapping destName="devicetype" destValue="mobile" sourceName="platform" sourceValue="ios"/>
	</derivedMetadataMappings>

	<!-- Strip Events - these delimit frames which will be excluded from stats computation -->
	<!-- Note: order is important, since these are stripped in the order listed (with no handling for overlapping events) -->
	<csvEventsToStrip>
		<eventPair begin="App_WillDeactivate" end="App_HasReactivated"/>
	</csvEventsToStrip>



	<!-- Preset summary tables -->
	<summaryTableColumnFormatInfo>
		<columnInfo name="count" autoColorize="lowIsBad"/>
        <columnInfo name="framecount" autoColorize="lowIsBad"/>
		<columnInfo name="DynamicResolutionPercentage*" autoColorize="lowIsBad"/>
		<columnInfo name="MemoryFreeMB*" autoColorize="lowIsBad"/>
		<columnInfo name="Total Time*" autoColorize="lowIsBad"/>
	</summaryTableColumnFormatInfo>

	<!--
	<sharedSummaries>
		<summary refName="psos" type="event" title="PSOs" metadataKey="PSOs">
			<events>PSO/*</events>
			<colourThresholds>0, 1, 2, 3</colourThresholds>
		</summary>	

		<summary refName="textureStreaming" type="boundedstatvalues" title="Texture Streaming" useUnstrippedCsvStats="1">
			<column name="% Time textures over budget" formula="percentunderthreshold" threshold="15" metadataKey="texturestreaming_texturesOverPercent" stat="texturestreaming/cachedmips">
				<colourThresholds>0,5,20,100</colourThresholds>
			</column>
			<column name="Cached mips (avg)" formula="average" threshold="15" metadataKey="textureStreaming_cachedMipsAvg" stat="texturestreaming/cachedmips">
				<colourThresholds>300,200,100,50</colourThresholds>
			</column>
			<column name="Streaming pool (avg)" formula="average" threshold="15" metadataKey="textureStreaming_streamingPoolAvg" stat="texturestreaming/streamingpool"/>
			<column name="Wanted mips (avg)" formula="average" threshold="15" metadataKey="textureStreaming_wantedMipsAvg" stat="texturestreaming/wantedmips"/>
			<column name="Temporary pool" formula="average" threshold="15" stat="texturestreaming/temporarypool"/>
			<column name="Safety pool" formula="average" threshold="15" stat="texturestreaming/safetypool"/>
		</summary>	
	</sharedSummaries>
	-->

	<!-- Preset summary tables -->
	<!-- In bulk mode (-csvdir), summary table row data is produced for each CSV. This includes data from all active summaries -->
	<!-- The preset Summary Tables below can be used to provide a filtered and sorted view of that data (see -summaryTable). -->
	<summaryTables>
		<summaryTable name="all" rowSort="buildversion,Platform,deviceprofile">
			<filter>buildversion,testbuildversion,Platform,DeviceProfile,scalabilitylevel,*</filter>
		</summaryTable>

		<summaryTable name="default" rowSort="buildversion,testbuildversion,playlistid,Platform,deviceprofile,scalabilitylevel,TargetFPS">
			<filter>Platform,DeviceProfile,scalabilitylevel,Report,TargetFPS,MVP*,Hitches/Min,HitchTimePercent,FrameTime*,GamethreadTime*,Renderthreadtime*,RHIThreadTime*,GPUTime*,MemoryFreeMB Min,PhysicalUsedMB Max,DynamicResolution*,RHI/Drawcalls*,Basic/TicksQueued*,Total Time*,Csv File,buildversion,testbuildversion,testbuildversion,config,playlistid,framecount*,syncloads</filter>
		</summaryTable>

		<summaryTable name="memory" rowSort="buildversion,Platform,deviceprofile,scalabilitylevel">
			<filter>MemoryFreeMB*,PhysicalUsedMB*,Total Time*</filter>
		</summaryTable>

		<summaryTable name="condensed" rowSort="buildversion,Platform,deviceprofile,scalabilitylevel,TargetFPS">
			<filter>DeviceProfile,scalabilitylevel,TargetFPS,MVP*,Hitches/Min,MemoryFreeMB Min,PhysicalUsedMB Max,GamethreadTime*,Renderthreadtime*,RHIThreadTime*,GPUTime*,DynamicResolution*,SyncLoads*</filter>
		</summaryTable>

		<summaryTable name="historical" rowSort="buildversion,platform,deviceprofile">
			<filter>buildversion,testbuildversion,Platform,DeviceProfile,scalabilitylevel, config,TargetFPS,testname,Report,MVP*,Hitches/Min,HitchTimePercent,FrameTime*,GamethreadTime*,Renderthreadtime*,RHIThreadTime*,GPUTime*,MemoryFreeMB Min,PhysicalUsedMB Max,DynamicResolution*,RHI/Drawcalls*,Ticks/PlayerPawns*,Basic/TicksQueued*,Total Time*,framecount*,fmsglogf/fmsglogfcount avg,syncloads</filter>
		</summaryTable>

		<!-- This table is used to produce the historic reports at the end of each Frosty.PerformanceReport Gauntlet test. -->
		<summaryTable name="autoPerfReportStandard" rowSort="buildversion,testbuildversion,Platform,deviceprofile" scrollableFormatting="1" reverseSortRows="1">
			<filter>buildversion,testbuildversion,DeviceProfile,config,count,testname,TargetFPS,MVP*,Hitches/Min,HitchTimePercent,FrameTime*,GamethreadTime*,Renderthreadtime*,RHIThreadTime*,GPUTime*,MemoryFreeMB Min,PhysicalUsedMB Max,DynamicResolution*,Total Time*,RHI/*,Basic/TicksQueued*,Ticks/Vehicles*,framecount*,Csv File,Report</filter>
		</summaryTable>

	</summaryTables>

	<reporttypes reportGraphsFile="ReportGraphs.xml">

		<!-- 60fps report -->
		<!-- This report is auto-detected -->
		<reporttype name="Default60fps" title="60FPS Performance Report" vsync="1">
			<autodetection requiredstats="frametime">
				<requiredmetadata key="targetframerate" allowedValues="60" stopIfKeyFound="1" ignoreIfKeyNotFound="1"/>
			</autodetection>
			
			<metadataToShow>buildversion,testbuildversion,platform,deviceprofile,csvfilename,rhiname,desktophwinfo,testname</metadataToShow>

			<summary type="fpschart" fps="60" hitchThreshold="60" useEngineHitchMetric="1">
				<stats>Frametime,GameThreadtime,RenderThreadtime,RHIThreadTime,GPUtime,RHI/Drawcalls,DynamicResolutionPercentage,MemoryFreeMB(min),PhysicalUsedMB(max),Basic/TicksQueued,fmsglogf/fmsglogfcount</stats>
				<colourThresholds stat="Hitches/Min"> 2, 3, 5, 6</colourThresholds>
				<colourThresholds stat="HitchTimePercent"> 0.1, 0.5, 1, 2</colourThresholds>
				<colourThresholds stat="MVP60"> 2, 3, 4, 5</colourThresholds>
				<colourThresholds stat="MemoryFreeMB(min)">400,300,200,50</colourThresholds>
				<colourThresholds stat="DynamicResolutionPercentage">90,80,70,60</colourThresholds>
				<colourThresholds stat="Frametime"> 16.7, 17.5, 19, 22.5 </colourThresholds>
				<colourThresholds stat="GameThreadtime"> 16.7, 17.5, 19, 22.5 </colourThresholds>
				<colourThresholds stat="RHIThreadTime"> 16.7, 17.5, 19, 22.5 </colourThresholds>
				<colourThresholds stat="RenderThreadtime"> 16.7, 17.5, 19, 22.5 </colourThresholds>
				<colourThresholds stat="GPUtime"> 16.7, 17.5, 19, 22.5 </colourThresholds>
				<colourThresholds stat="Basic/TicksQueued"> 150, 200, 250, 300 </colourThresholds>
				<colourThresholds stat="fmsglogf/fmsglogfcount">0.01,0.1,0.2,0.5</colourThresholds>
			</summary>

			<summary type="hitches">
				<stats>Frametime,GameThreadtime,RenderThreadtime,RHIThreadTime,GPUtime</stats>
				<colourThresholds>33, 20, 16.6, 16.2</colourThresholds>
				<hitchThresholds>60, 150, 250, 500, 750, 1000, 2000</hitchThresholds>
			</summary>

			<!--
			<summaryRef name="mapOverlay"/>
			<summaryRef name="syncLoads"/>
			<summaryRef name="textureStreaming"/>
			<summaryRef name="psos"/>
			-->

			<graph title="Stat Unit" budget="16.66" />
			<graph title="Stat Unit Raw" budget="16.66" />
			<graph title="Gamethread Breakdown" budget="16.66"/>
			<graph title="Gamethread Breakdown Raw" budget="16.66"/>
			<graph title="Gamethread Waits" budget="2"/>
			<graph title="Gamethread Physics" budget="2"/>
			<graph title="Gamethread Breakdown Giant" budget="16.66" external="1"/>
			<graph title="Ticks Breakdown" budget="200"/>
			<graph title="Actor Counts" budget="0"/>
			<graph title="Cmds DayaLayers" budget="300"/>
			<graph title="Physical Memory" budget="300"/>
			<graph title="Physical Memory Over Budget" budget="300"/>
			<graph title="Dynamic Resolution" budget="80"/>
			<graph title="GPU Breakdown" budget="16.66" />
			<graph title="GPU Breakdown Raw" budget="16.66" />
			<graph title="D3D12 Video Memory"  budget="0" />
			<graph title="Niagara Breakdown" budget="33.33"/>
			<graph title="Light Counts" budget="0" />
			<graph title="RHI Drawcalls" budget="3000" />
			<graph title="Renderthread Breakdown" budget="16.66"/>
			<graph title="Renderthread Breakdown Raw" budget="16.66"/>
			<graph title="FScene Breakdown" budget="33.33"/>
			<graph title="FScene Breakdown Raw" budget="33.33"/>
			<graph title="Render Targets Peak(MB)" budget="0" />
			<graph title="Textures Peak(MB)" budget="0" />
			<graph title="Virtual Texture" budget="0" />
			<graph title="Animation Budget" budget="1.5" />
			<graph title="Texture Streaming" budget="0" />
			<graph title="Log Counts" budget="0" />
			<graph title="CSV Profiler" budget="0" />
		</reporttype>

		<!-- VP 24fps report type -->
		<reporttype name="VP24fps" title="VP 24FPS Performance Report" vsync="0">
			<autodetection requiredstats="frametime">
				<requiredmetadata key="targetframerate" allowedValues="24" stopIfKeyFound="1" ignoreIfKeyNotFound="1"/>
			</autodetection>
			
			<metadataToShow>buildversion,testbuildversion,platform,deviceprofile,csvfilename,rhiname,desktophwinfo,testname</metadataToShow>

			<summary type="fpschart" fps="24" hitchThreshold="24" useEngineHitchMetric="1">
				<stats>Frametime,GameThreadtime,RenderThreadtime,RHIThreadTime,GPUtime,RHI/Drawcalls,DynamicResolutionPercentage,MemoryFreeMB(min),PhysicalUsedMB(max),Basic/TicksQueued,fmsglogf/fmsglogfcount</stats>
				<colourThresholds stat="Hitches/Min"> 2, 3, 5, 6</colourThresholds>
				<colourThresholds stat="HitchTimePercent"> 0.1, 0.5, 1, 2</colourThresholds>
				<colourThresholds stat="MVP24"> 2, 3, 4, 5</colourThresholds>
				<colourThresholds stat="MemoryFreeMB(min)">400,300,200,50</colourThresholds>
				<colourThresholds stat="DynamicResolutionPercentage">90,80,70,60</colourThresholds>
				<colourThresholds stat="Frametime"> 41.66, 44, 48, 56 </colourThresholds>
				<colourThresholds stat="GameThreadtime"> 41.66, 44, 48, 56 </colourThresholds>
				<colourThresholds stat="RHIThreadTime"> 41.66, 44, 48, 56 </colourThresholds>
				<colourThresholds stat="RenderThreadtime"> 41.66, 44, 48, 56 </colourThresholds>
				<colourThresholds stat="GPUtime"> 41.66, 44, 48, 56 </colourThresholds>
				<colourThresholds stat="Basic/TicksQueued"> 150, 200, 250, 300 </colourThresholds>
				<colourThresholds stat="fmsglogf/fmsglogfcount">0.01,0.1,0.2,0.5</colourThresholds>
			</summary>

			<summary type="hitches">
				<stats>Frametime,GameThreadtime,RenderThreadtime,RHIThreadTime,GPUtime</stats>
				<colourThresholds>82, 55, 41.6, 38.0</colourThresholds>
				<hitchThresholds>60, 150, 250, 500, 750, 1000, 2000</hitchThresholds>
			</summary>

			<!--
			<summaryRef name="mapOverlay"/>
			<summaryRef name="syncLoads"/>
			<summaryRef name="textureStreaming"/>
			<summaryRef name="psos"/>
			-->

			<graph title="Stat Unit" budget="41.66" />
			<graph title="Stat Unit Raw" budget="41.66" />
			<graph title="Gamethread Breakdown" budget="41.66"/>
			<graph title="Gamethread Breakdown Raw" budget="41.66"/>
			<graph title="Gamethread Waits" budget="2"/>
			<graph title="Gamethread Physics" budget="2"/>
			<graph title="Gamethread Breakdown Giant" budget="41.66" external="1"/>
			<graph title="Ticks Breakdown" budget="200"/>
			<graph title="Actor Counts" budget="0"/>
			<graph title="Cmds DayaLayers" budget="300"/>
			<graph title="Physical Memory" budget="300"/>
			<graph title="Physical Memory Over Budget" budget="300"/>
			<graph title="Dynamic Resolution" budget="80"/>
			<graph title="GPU Breakdown" budget="41.66" />
			<graph title="GPU Breakdown Raw" budget="41.66" />
			<graph title="D3D12 Video Memory"  budget="0" />
			<graph title="Niagara Breakdown" budget="41.66"/>
			<graph title="Light Counts" budget="0" />
			<graph title="RHI Drawcalls" budget="3000" />
			<graph title="Renderthread Breakdown" budget="41.66"/>
			<graph title="Renderthread Breakdown Raw" budget="41.66"/>
			<graph title="FScene Breakdown" budget="41.66"/>
			<graph title="FScene Breakdown Raw" budget="41.66"/>
			<graph title="Render Targets Peak(MB)" budget="0" />
			<graph title="Textures Peak(MB)" budget="0" />
			<graph title="Virtual Texture" budget="0" />
			<graph title="Animation Budget" budget="1.5" />
			<graph title="Texture Streaming" budget="0" />
			<graph title="Log Counts" budget="0" />
			<graph title="CSV Profiler" budget="0" />
		</reporttype>
	</reporttypes>
</root>