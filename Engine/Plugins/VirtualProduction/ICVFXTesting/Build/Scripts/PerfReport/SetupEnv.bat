set CSVToSVGPath=%~dp0..\..\..\..\..\..\..\..\Engine\Binaries\DotNet\CsvTools\CsvToSVG
set CSVToolsPath=%~dp0..\..\..\..\..\..\..\..\Engine\Binaries\DotNet\CsvTools\