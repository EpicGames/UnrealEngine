<?xml version='1.0' ?>
<BuildGraph xmlns="http://www.epicgames.com/BuildGraph" xmlns:xsi="http://www.w3.org/2001/XMLSchema-instance" xsi:schemaLocation="http://www.epicgames.com/BuildGraph ../../Schema.xsd" >

	<Property Name="ProjectTestName" Value="$(ProjectName)Test"/>
		
	<Option Name="WithEditorAutoTest" Restrict="true|false" DefaultValue="false" Description="Whether to run the AutoTest on the editor build." />
	<Option Name="WithEditorAutomationPIETest" Restrict="true|false" DefaultValue="false" Description="Whether to run UE.EditorAutomation(RunTest=Project.Maps.PIE)." />
	<Option Name="WithEditorAutomationAll" Restrict="true|false" DefaultValue="false" Description="Whether to run UE.EditorAutomation(RunTest=Project.)." />

	<Option Name="WithAutoTest" Restrict="true|false" DefaultValue="false" Description="Whether to run AutoTest on the target builds." />
	<Option Name="WithPerformanceReportTest" Restrict="true|false" DefaultValue="true" Description="Whether to run the PerformanceReport tests." />
	<Option Name="WithMemoryReportTest" Restrict="true|false" DefaultValue="false" Description="Whether to run the MemoryReport test." />
	<Option Name="WithTargetAutomationMapsCycle" Restrict="true|false" DefaultValue="false" Description="Whether to run UE.TargetAutomation(RunTest=Project.Maps.Cycle)." />
	<Option Name="WithTargetAutomationAll" Restrict="true|false" DefaultValue="false" Description="Whether to run UE.TargetAutomation(RunTest=Project.)." />

	<!-- For on-set testing, there will be custom approach -->
	<Option Name="TestNameExt" DefaultValue="" Description="Extended test name (sub-test name for on-stage runs)" />
	<Option Name="TraceFileName" DefaultValue="" Description="Explicit trace file name" />
	<Option Name="TestBuildName" DefaultValue="CL-0" Description="Explicit build name" />
	<Option Name="Fullscreen" Restrict="true|false" DefaultValue="false" Description="Should run fullscreen" />
	<Option Name="Offscreen" Restrict="true|false" DefaultValue="false" Description="Should run offscreen" />
	<Option Name="WndLocX" DefaultValue="0" Description="Window X position" />
	<Option Name="WndLocY" DefaultValue="0" Description="Window Y position" />
	<Option Name="WndResX" DefaultValue="1920" Description="Window X size" />
	<Option Name="WndResY" DefaultValue="1080" Description="Window Y size" />
	<Option Name="SummaryReportType" DefaultValue="" Description="The type/preset of the final test reports" />
	<Option Name="IsOnStagePerfTest" Restrict="true|false" DefaultValue="false" Description="Whether it's running on a real stage" />
	<Do If="$(IsOnStagePerfTest)">
		<Property Name="EditorTestList" Value="ICVFXTest.PerformanceReport+" If="$(WithPerformanceReportTest)" />
	</Do>

	<!-- Hack solution to set target test list for BuildAndTest.xml -->
	<Do If="'$(EditorTestList)' == ''">
		<Property Name="TargetTestList" Value="$(EditorTestList)ICVFXTest.MemoryReport+" If="$(WithMemoryReportTest)" />
		<Property Name="EditorTestList" Value="$(EditorTestList)ICVFXTest.PerformanceReport+" If="$(WithPerformanceReportTest)" />
		<Property Name="EditorTestList" Value="$(EditorTestList)ICVFXTest.PerformanceReport_MGPU+" If="$(WithPerformanceReportTest)" />
		<Property Name="EditorTestList" Value="$(EditorTestList)ICVFXTest.PerformanceReport_Nanite+" If="$(WithPerformanceReportTest)" />
		<Property Name="EditorTestList" Value="$(EditorTestList)ICVFXTest.PerformanceReport_NaniteLumen+" If="$(WithPerformanceReportTest)" />
		<Property Name="EditorTestList" Value="$(EditorTestList)ICVFXTest.PerformanceReport_Vulkan+" If="$(WithPerformanceReportTest)" />
	</Do>

	<!-- How long to give Gauntlet before a test is considered timed out (some tests will time out earlier if they detect inactivity)-->
	<Option Name="GauntletTimeout" DefaultValue="900" Description="Max running time of a test in Gauntlet"/>
	
	
	<!-- Device option overrides -->
	<Option Name="GauntletDevicePool" DefaultValue="UE5" Description="Gauntlet Device Pool to use to run target platform tests"/>
	<Option Name="AlternateDevicePoolPlatforms" DefaultValue="" Description="Platforms that need to use the Alternate Gauntlet Device Pool"/>
	
	<!-- Test specific options -->
	<Option Name="DisplayConfigPath" DefaultValue="" Description="The path to the .ndisplay file used for launching the cluster."/>
	<Option Name="DisplayClusterNodeName" DefaultValue="" Description="The name of the ndisplay node to launch."/>
	<Option Name="ViewportScreenPercentage" DefaultValue="100" Description="The viewport screen percentage to use"/>
	<Option Name="WithNanite" Restrict="true|false" DefaultValue="false" Description="Whether to use nanite when testing"/>
	<Option Name="WithLumen" Restrict="true|false" DefaultValue="false" Description="Whether to use lumen when testing"/>
	<Option Name="WithD3DDebug" Restrict="true|false" DefaultValue="false" Description="Whether to add d3ddebug command line argument."/>
	<Option Name="WithGPUCrashDebugging" Restrict="true|false" DefaultValue="false" Description="Whether to add gpucrashdebugging  command line argument."/>
	<Option Name="MaxGPUCount" DefaultValue="1" Description="Number of GPUs to use for the test."/>
	<Option Name="MaxRunCount" DefaultValue="1" Description="Number of iterations to do."/>
	<Option Name="WithRDGImmediate" Restrict="true|false" DefaultValue="false" Description="Whether to run with -rdgimmediate"/>
	<Option Name="WithNoRHIThread" Restrict="true|false" DefaultValue="false" Description="Whether to run with -norhithread"/>
	<Option Name="WithVirtualShadowMaps" Restrict="true|false" DefaultValue="true" Description="Whether to use virtual shadow maps"/>
	<Option Name="WithTrace" Restrict="true|false" DefaultValue="false" Description="Whether to run with unreal insights."/>
	
	<!-- PRS Specific Options -->
	<Option Name="NoLocalReports" Restrict="true|false" DefaultValue="false" Description="Whether to generate the local test reports."/>
	<Option Name="SkipPerfReportServer" Restrict="true|false" DefaultValue="false" Description="Whether to skip uploading gauntlet perf data to the PRS."/>
	<Option Name="PerfReportServerImportDir" DefaultValue="" Description="Override the server import directory for PRS."/>
	<Option Name="IsDevelopment" Restrict="true|false" DefaultValue="false" Description="Whether running in development or not."/>
	
	<Property Name="Platform" Value="Win64"/>

	<!-- Automation uses its own network output directory -->
	<Property Name="AutomationNetworkRootDir" Value="\\epicgames.net\root\Builds\Automation\Reports\$(ProjectName)Perf"/>
	<Property Name="AutomationNetworkRootDir" Value="/Volumes/Builds/Automation/Reports/$(ProjectName)Perf" If="'$(HostPlatform)' != 'Win64'"/>
	<Property Name="AutomationLocalOutputDir" Value="$(RootDir)/LocalBuilds/Reports/$(ProjectName)Perf"/>
	<Property Name="AutomationLocalOutputDir" Value="$(RootDir)/LocalBuilds/Reports/$(ProjectName)/$(TestNameExt)/$(DisplayClusterNodeName)" If="$(IsOnStagePerfTest)" />

	<!-- Outputs for this run keeping slashes flipped the right way because it shows up on the webapp this way -->
	<Property Name="AutomationNetworkOutputDir" Value="$(AutomationNetworkRootDir)/$(BuildNamePath)"/>
	<Property Name="AutomationNetworkOutputDir" Value="$(AutomationNetworkRootDir)/$(BuildNamePath)" If="'$(HostPlatform)' != 'Win64'"/>
	<Property Name="AutomationOutputDir" Value="$(AutomationLocalOutputDir)"/>
	<Property Name="AutomationOutputDir" Value="$(AutomationNetworkOutputDir)" If="$(IsBuildMachine)"/>
	
	<Property Name="SummaryReportPath" Value="$(AutomationLocalOutputDir)"/>
	<Property Name="SummaryReportPath" Value="$(AutomationNetworkRootDir)" If="$(IsBuildMachine)"/>
	
	<Property Name="ReportCacheDirRoot" Value="$(SummaryReportPath)/PerfCache" />
	
	<Property Name="TestOutputDir" Value="$(AutomationOutputDir)/$(ProjectName)$(Platform)$(ProjectName)Perf"/>
	<Property Name="LogOutputParams" Value="-logdir=&quot;$(TestOutputDir)&quot;"/>
	
	<Property Name="TraceRootFolder" Value="$(TestOutputDir)/$(BuildNamePath)" />
	<Property Name="TraceRootFolder" Value="$(TestOutputDir)" If="$(IsOnStagePerfTest)" />	

	<Property Name="BuildName" Value="$(EscapedBranch)-CL-$(Change)$(PreflightSuffix)" />
	<Property Name="BuildName" Value="$(TestBuildName)" If="$(IsOnStagePerfTest)" />	
		
	<Property Name="ExtraGauntletCommonArgs" Value="-screenshotperiod=0 -ICVFXTest.StatCommands -ICVFXTest.MaxRunCount=$(MaxRunCount) $(LogOutputParams)" />
	<Property Name="ExtraGauntletEditorTestArgs" Value="-game" />

	<Do If="$(IsOnStagePerfTest)">
		<Property Name="ExtraGauntletEditorTestArgs" Value="$(ExtraGauntletEditorTestArgs) -RetainDeviceArtifacts" />
	</Do>

	<Property Name="ExtraGauntletEditorTestArgs" Value="$(ExtraGauntletEditorTestArgs) -ICVFXTest.IsBuildMachine=$(IsBuildMachine)" />
	<Property Name="ExtraGauntletEditorTestArgs" Value="$(ExtraGauntletEditorTestArgs) -ICVFXTest.TestName=$(ProjectTestName)" />
	<Property Name="ExtraGauntletEditorTestArgs" Value="$(ExtraGauntletEditorTestArgs) -ICVFXTest.MapOverride=$(MapOverride)" />
	<Property Name="ExtraGauntletEditorTestArgs" Value="$(ExtraGauntletEditorTestArgs) -ICVFXTest.DisplayConfig=$(DisplayConfigPath)" />
	<Property Name="ExtraGauntletEditorTestArgs" Value="$(ExtraGauntletEditorTestArgs) -ICVFXTest.ViewportScreenPercentage=$(ViewportScreenPercentage)" />
	<Property Name="ExtraGauntletEditorTestArgs" Value="$(ExtraGauntletEditorTestArgs) -ICVFXTest.Fullscreen" If="$(Fullscreen)" />
	<Property Name="ExtraGauntletEditorTestArgs" Value="$(ExtraGauntletEditorTestArgs) -ICVFXTest.Offscreen" If="$(Offscreen)" />
	<Property Name="ExtraGauntletEditorTestArgs" Value="$(ExtraGauntletEditorTestArgs) -ICVFXTest.WndLocX=$(WndLocX)" />
	<Property Name="ExtraGauntletEditorTestArgs" Value="$(ExtraGauntletEditorTestArgs) -ICVFXTest.WndLocY=$(WndLocY)" />
	<Property Name="ExtraGauntletEditorTestArgs" Value="$(ExtraGauntletEditorTestArgs) -ICVFXTest.WndResX=$(WndResX)" />
	<Property Name="ExtraGauntletEditorTestArgs" Value="$(ExtraGauntletEditorTestArgs) -ICVFXTest.WndResY=$(WndResY)" />
	<Property Name="ExtraGauntletEditorTestArgs" Value="$(ExtraGauntletEditorTestArgs) -ICVFXTest.DisplayClusterNodeName=$(DisplayClusterNodeName)" />
	<Property Name="ExtraGauntletEditorTestArgs" Value="$(ExtraGauntletEditorTestArgs) -ICVFXTest.D3DDebug" If="$(WithD3DDebug)"/>
	<Property Name="ExtraGauntletEditorTestArgs" Value="$(ExtraGauntletEditorTestArgs) -ICVFXTest.GPUCrashDebugging" If="$(WithGPUCrashDebugging)"/>
	<Property Name="ExtraGauntletEditorTestArgs" Value="$(ExtraGauntletEditorTestArgs) -ICVFXTest.Nanite" If="$(WithNanite)"/>
	<Property Name="ExtraGauntletEditorTestArgs" Value="$(ExtraGauntletEditorTestArgs) -ICVFXTest.Lumen" If="$(WithLumen)"/>
	<Property Name="ExtraGauntletEditorTestArgs" Value="$(ExtraGauntletEditorTestArgs) -ICVFXTest.MaxGPUCount=$(MaxGPUCount)"/>
	<Property Name="ExtraGauntletEditorTestArgs" Value="$(ExtraGauntletEditorTestArgs) -ICVFXTest.SummaryReportType=$(SummaryReportType)" If="'$(SummaryReportType)' != ''" />
	<Property Name="ExtraGauntletEditorTestArgs" Value="$(ExtraGauntletEditorTestArgs) -ICVFXTest.SummaryReportPath=$(SummaryReportPath)"/>
	<Property Name="ExtraGauntletEditorTestArgs" Value="$(ExtraGauntletEditorTestArgs) -ICVFXTest.RDGImmediate" If="$(WithRDGImmediate)"/>
	<Property Name="ExtraGauntletEditorTestArgs" Value="$(ExtraGauntletEditorTestArgs) -ICVFXTest.NoRHIThread" If="$(WithNoRHIThread)"/>
	<Property Name="ExtraGauntletEditorTestArgs" Value="$(ExtraGauntletEditorTestArgs) -ICVFXTest.DisableVirtualShadowMaps" If="!$(WithVirtualShadowMaps)"/>
	<Property Name="ExtraGauntletEditorTestArgs" Value="$(ExtraGauntletEditorTestArgs) -ICVFXTest.EnableTrace" If="$(WithTrace)"/>
	<Property Name="ExtraGauntletEditorTestArgs" Value="$(ExtraGauntletEditorTestArgs) -ICVFXTest.TraceRootFolder=$(TraceRootFolder)" If="$(WithTrace)"/>
	<Property Name="ExtraGauntletEditorTestArgs" Value="$(ExtraGauntletEditorTestArgs) -ICVFXTest.TraceFileName=$(TraceFileName)" If="'$(TraceFileName)' != ''"/>
	<Property Name="ExtraGauntletEditorTestArgs" Value="$(ExtraGauntletEditorTestArgs) -ICVFXTest.PerfCacheFolder=$(ReportCacheDirRoot)"/>
	<Property Name="ExtraGauntletEditorTestArgs" Value="$(ExtraGauntletEditorTestArgs) -ICVFXTest.NoLocalReports" If="$(NoLocalReports)"/>
	<Property Name="ExtraGauntletEditorTestArgs" Value="$(ExtraGauntletEditorTestArgs) -ICVFXTest.SkipPerfReportServer" If="$(SkipPerfReportServer)"/>
	<Property Name="ExtraGauntletEditorTestArgs" Value="$(ExtraGauntletEditorTestArgs) -ICVFXTest.IsDevelopment" If="$(IsDevelopment)"/>
	<Property Name="ExtraGauntletEditorTestArgs" Value="$(ExtraGauntletEditorTestArgs) -ICVFXTest.PerfReportServerImportDir=$(PerfReportServerImportDir)"/>
	<Property Name="ExtraGauntletEditorTestArgs" Value="$(ExtraGauntletEditorTestArgs) -ICVFXTest.BuildName=$(BuildName)" />

	<!-- Common Gauntlet settings -->
	<Include Script="../../../../../Engine/Build/Graph/Tasks/Inc/GauntletSettings.xml" />

</BuildGraph>