<?xml version='1.0' ?>
<BuildGraph xmlns="http://www.epicgames.com/BuildGraph" xmlns:xsi="http://www.w3.org/2001/XMLSchema-instance" xsi:schemaLocation="http://www.epicgames.com/BuildGraph ../Schema.xsd" >

	<!-- todo can this be disbaled? -->
	<Property Name="Versioned" Value="$(IsBuildMachine)" />
	<Property Name="PublishHashedStagingData" Value="$(IsBuildMachine)" />

	<!-- Turn off BuildAndTest defaults so we can set out own -->
	<Property Name="WithBATDefaults" Value="false" />

	<!-- Set defaults for people running this script with no arguments. These can still be overriden with set:EditorPlatforms= etc) -->
	<Property Name="DefaultEditorPlatforms" Value="Win64" />
	<Property Name="DefaultTargetPlatforms" Value="" /> <!-- We dont need a cooked build -->
	<Property Name="DefaultTargetConfigurations" Value="Development" />
	<Property Name="DefaultEditorTestList" Value="" />
	<Property Name="DefaultTargetTestList" Value="" />
	<Property Name="NetworkTempRootOverride" Value="" />
	<Property Name="NetworkPublishRootOverride" Value="" />
	<Property Name="NetworkReportRootOverride" Value="" />
	<Property Name="SymbolServerPath" Value="" />

	<Property Name="ExtraStageAndPackageArguments" Value="-compressed -CrashReporter" />
	
	<!-- If a build machine, turn off all defaults. Settings should be specified via the website UI -->
	<Do If="$(IsBuildMachine)">
		<Property Name="DefaultEditorPlatforms" Value="" />
		<Property Name="DefaultTargetPlatforms" Value=""/>
		<Property Name="DefaultTargetConfigurations" Value="Development"/>
		<Property Name="DefaultEditorTestList" Value="" />
		<Property Name="DefaultTargetTestList" Value="" />
	</Do>

	<Property Name="TestAgentOverride" Value="EditorPerfWin64" />
	<!-- Common Setting Overrides -->
	
	<Property Name="ICVFXTestingPath" Value="$(RootDir)/Engine/Plugins/VirtualProduction/ICVFXTesting/Scripts"/>
	<Property Name="GauntletSettingsFile" Value="$(ICVFXTestingPath)/GauntletSettings.xml" />

	<!-- This will declare an aggregate called BuildAndTest $(ProjectName) -->
	<Include Script="$(RootDir)/Engine/Build/Graph/Tasks/BuildAndTestProject.xml"/>
</BuildGraph>