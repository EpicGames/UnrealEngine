import webbrowser, os
import sys, getopt

def main(argv):
    print("Opening folder")
    input_folder = argv[0]
    input_folder = os.path.realpath(input_folder)
    os.system("start " + os.path.join(input_folder, "HistoricReport_14Days_SummaryBase_Email.html"))

if __name__ == "__main__":
    main(sys.argv[1:])