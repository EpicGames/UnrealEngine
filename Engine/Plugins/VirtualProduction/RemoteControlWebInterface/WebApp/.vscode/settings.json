{
  "files.exclude": {
    "**/.vs": true,
    "**/node_modules": true,
    "**/build/**": true,
    "**/Server/public/**": true,
  },
  "editor.tabSize": 2,
  "editor.insertSpaces": true,
  "editor.detectIndentation": false,
  "terminal.integrated.cursorStyle": "line",
  "terminal.integrated.copyOnSelection": true,
}