@echo off

call :DEL_PROJECT_FILES		.\
call :DEL_FOLDERS 			.\node_modules
call :DEL_PROJECT_FILES		.\Client
call :DEL_FOLDERS 			.\Client\build
call :DEL_FOLDERS 			.\Client\node_modules
call :DEL_PROJECT_FILES		.\Server
call :DEL_FOLDERS 			.\Server\build
call :DEL_FOLDERS 			.\Server\node_modules
call :DEL_FOLDERS 			.\Server\public
goto :EOF



:DEL_FOLDERS
REM Using Robocopy instead of rmdir
REM as node_modules may contains really long file path, and rmdir fails to delete them
REM echo Deleting %~1
if not exist empty\ mkdir empty
robocopy empty "%~1" /mir > nul
rmdir /q /s "%~1"
rmdir /q /s empty
goto :EOF


:DEL_PROJECT_FILES
REM Files are marked as readonly because of perforce
REM Node v16 fails to install dependencies because it tries to overwrite the file
REM if we force node to skip it, earlier versions of node will fail if we do that
REM so we have original files prefixed with _, at build they are being copied and the readonly attribute is removed
if exist "%~1\package.json"         del /f /q "%~1\package.json"
if exist "%~1\package-lock.json"    del /f /q "%~1\package-lock.json"
goto :EOF