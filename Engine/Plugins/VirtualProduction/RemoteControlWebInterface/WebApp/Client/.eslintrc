{
  "extends": ["react-app"],
  "rules": {
    "semi": ["error", "always", { "omitLastInOneLineBlock": true}],
    "no-console": "off",
    "no-unused-vars": "off",
    "no-case-declarations": "off",
    "no-empty": ["error", { "allowEmptyCatch": true }],
    "default-case": "off",
    "no-mixed-operators": "off"
  },
  "parserOptions": {
    "ecmaFeatures": {
      "legacyDecorators": true
    }
  }
}