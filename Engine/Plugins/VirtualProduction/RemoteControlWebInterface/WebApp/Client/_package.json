{
  "name": "conductor-client",
  "scripts": {
    "build": "react-app-rewired build",
    "start": "cross-env PORT=7000 react-app-rewired start"
  },
  "dependencies": {
    "@fortawesome/fontawesome-svg-core": "^1.2.30",
    "@fortawesome/free-regular-svg-icons": "^6.0.0",
    "@fortawesome/free-solid-svg-icons": "^5.14.0",
    "@fortawesome/react-fontawesome": "^0.1.11",
    "dot-prop-immutable": "^2.1.0",
    "lodash": "^4.17.19",
    "react": "^16.13.1",
    "react-beautiful-dnd": "^13.1.0",
    "react-dom": "^16.13.1",
    "react-redux": "^7.2.0",
    "react-virtuoso": "^2.2.1",
    "redux": "^4.0.5",
    "redux-act": "^1.8.0",
    "redux-thunk": "^2.3.0",
    "socket.io-client": "^2.3.0"
  },
  "devDependencies": {
    "@babel/plugin-proposal-class-properties": "^7.10.4",
    "@babel/plugin-proposal-decorators": "^7.13.5",
    "@babel/plugin-proposal-nullish-coalescing-operator": "^7.10.4",
    "@babel/plugin-proposal-optional-chaining": "^7.10.4",
    "@types/dot-prop-immutable": "^1.5.1",
    "@types/less": "^3.0.2",
    "@types/lodash": "^4.14.158",
    "@types/react": "^16.9.43",
    "@types/react-beautiful-dnd": "^13.0.0",
    "@types/react-dom": "^16.9.8",
    "@types/react-redux": "^7.1.9",
    "@types/socket.io-client": "^1.4.33",
    "cross-env": "^7.0.2",
    "customize-cra": "^1.0.0",
    "less": "^4.1.1",
    "less-loader": "^7.0.0",
    "react-app-rewired": "^2.1.6",
    "react-scripts": "^3.4.1",
    "typescript": "^3.9.7"
  },
  "browserslist": {
    "production": [
      ">0.2%",
      "not dead",
      "not op_mini all"
    ],
    "development": [
      "last 1 chrome version",
      "last 1 firefox version",
      "last 1 safari version"
    ]
  }
}
