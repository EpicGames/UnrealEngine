const {
  override,
  addBabelPlugin,
  addLessLoader,
  addWebpackAlias,
  useEslintRc,
  enableEslintTypescript,
  overrideDevServer,
  addDecoratorsLegacy,
} = require('customize-cra');
const path = require('path');
const _ = require('lodash');


module.exports = {
  webpack: override(
    addDecoratorsLegacy(),
    useEslintRc(),
    enableEslintTypescript(),
    addBabelPlugin('@babel/plugin-proposal-nullish-coalescing-operator'),
    addBabelPlugin('@babel/plugin-proposal-optional-chaining'),
    addBabelPlugin('@babel/plugin-proposal-class-properties'),
    addLessLoader({ lessOptions: { javascriptEnabled: true } }),
    (config) => {
      const toRemove = ['GenerateSW'];
      for (let plugin of toRemove)
        _.remove(config.plugins, p => p.constructor.name == plugin);

      return config;
    },
  ),
  devServer: overrideDevServer(
    (config) => ({ ...config, proxy: { 
      '/api': 'http://localhost:7001',
     } })
  ),
};
