{
  "name": "Unreal Engine",
  "short_name": "Unreal Engine",
  "icons": [
    {
      "src": "/images/favicon/android-chrome-192x192.png?v=xQdXpAz40w",
      "sizes": "192x192",
      "type": "image/png"
    },
    {
      "src": "/images/favicon/android-chrome-256x256.png?v=xQdXpAz40w",
      "sizes": "256x256",
      "type": "image/png"
    },
    {
      "src": "/images/favicon/android-chrome-512x512.png?v=xQdXpAz40w",
      "sizes": "512x512",
      "type": "image/png"
    }
  ],
  "theme_color": "#ffffff",
  "background_color": "#ffffff",
  "display": "browser",
  "scope": "/",
  "start_url": "/"
}