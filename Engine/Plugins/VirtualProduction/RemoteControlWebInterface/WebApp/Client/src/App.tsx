import React from 'react';
import { connect } from 'react-redux';
import { _api, ReduxState } from './reducers';
import { ConnectionStatus, Tabs, Screen, Stack, RenameModal, AlertModal, IconModal, PropertiesDrawer, Login, SignalIcon, EmptyTab } from './components';
import { IPreset, IView, ITab, TabLayout, WidgetTypes, ICustomStackProperty, IPanelType, ScreenType, IPanel, ICustomStackListItem, IHistory, ConnectionSignal } from './shared';
import { fas, IconName } from '@fortawesome/free-solid-svg-icons';
import { BeforeCapture, DragDropContext, DropResult } from 'react-beautiful-dnd';
import { FontAwesomeIcon } from '@fortawesome/react-fontawesome';
import _ from 'lodash';

import './example/CustomWidget';


type PropsFromState = Partial<{
  keyCorrect?: boolean;
  connected?: boolean;
  loading?: boolean;
  isOpen?: boolean;
  view: IView;
  presets: IPreset[]
  preset: IPreset;
  signal?: ConnectionSignal;
  lockedUI: boolean,
  undo: IHistory[],
  redo: IHistory[],
  initialize?: () => void;
}>;

const mapStateToProps = (state: ReduxState): PropsFromState => ({
  connected: state.api.status.connected,
  keyCorrect: state.api.status.keyCorrect,
  loading: state.api.status.loading,
  isOpen: state.api.status.isOpen,
  presets: _.values(state.api.presets),
  preset: state.api.presets[state.api.preset],
  view: state.api.view,
  lockedUI: state.api.lockedUI,
  signal: state.api.status.signal,
  undo: state.api.undo,
  redo: state.api.redo,
});

const mapDispatchToProps = (dispatch): PropsFromState => ({
  initialize: async () => dispatch(_api.initialize()),
});

type DropElement = {
  id: string;
  path: string;
  type: string;
  accept: string[];
}

type DropBoundaries = {
  id: string;
  top: number;
  bottom: number;
}

type State = {
  tab: number;
  editable?: boolean;
  panel?: number;
  dragging?: string;
  droppable?: string;
  selected?: string;
  hoverTab?: string;
  vector?: ICustomStackProperty;
};


@(connect(mapStateToProps, mapDispatchToProps) as any)
export class App extends React.Component<PropsFromState, State> {

  state: State = {
    tab: 0,
    panel: 0,
  };

  appRef = React.createRef<HTMLDivElement>();
  panelRef = React.createRef<HTMLDivElement>();

  widget: any[] = null;
  dropElements: Record<string, DropElement> = {};
  dropBoundaries: DropBoundaries[] = [];
  hoverDelay = null;
  buildInfo: string = '';

  componentDidMount() {
    this.props.initialize();
    document.onpointermove = this.onPointerMove;
    this.getBuildInfo()
        .then(info => this.buildInfo = info);
  }

  componentDidUpdate(prevProps: PropsFromState, prevState: State) {
    const { preset } = this.props;
    const { editable } = this.state;

    if (preset?.Name !== prevProps.preset?.Name)
      this.setState({ tab: 0 });

    if (editable !== prevState.editable)
      this.setState({ selected: null });

    if (this.props.connected && !prevProps.connected)
      this.checkVersionMatch();
  }

  componentWillUnmount() {
    document.onpointermove = null;
  }

  getBuildInfo = async (): Promise<string> => {
    try {
      const req = await fetch('/version.json');
      return await req.text();
    } catch {
    }
  }

  checkVersionMatch = async () => {
    const buildInfo = await this.getBuildInfo();
    if (!this.buildInfo || !buildInfo)
      return;

    if (this.buildInfo === buildInfo)
      return;

    const reload = await AlertModal.show('The web app seems to be outdated, reload?');
    if (reload)
      document.location.reload();
  }

  onDropElementAdd = (id: string, path: string = '', accept: string[], type) => {
    this.dropElements[id] = { id, path, accept, type };

    return id;
  }

  onPointerMove = (e: PointerEvent) => {
    const { dragging, droppable } = this.state;

    if (!dragging)
      return;

    const elements = document.elementsFromPoint(e.clientX, e.clientY);
    const tab = elements.find(el => !!el?.getAttribute('data-prefix'));

    if (tab) {
      const prefix = tab.getAttribute('data-prefix');
      const id = tab.getAttribute('data-value');

      clearTimeout(this.hoverDelay);
      return this.hoverDelay = setTimeout(() => this.setState({ hoverTab: `${prefix}_${id}` }), 300);
    }

    clearTimeout(this.hoverDelay);

    const el = elements.find(el => !!el?.getAttribute('data-rbd-droppable-id'));
    const attr = el?.getAttribute('data-rbd-droppable-id');

    if (!attr || droppable === attr)
      return;

    const dropTypes = this.dropElements[attr]?.accept || [];
    const type = _.last(dragging.split('_'));

    if (dropTypes.includes(type) || _.first(dropTypes) === 'ALL')
      this.setState({ droppable: attr });
  }

  onKeyDown = async (e: React.KeyboardEvent) => {
    if (!e.ctrlKey && !e.metaKey && e.key !== 'Delete')
      return;

    const { tab, selected, editable } = this.state;

    if (e.key === 'Delete') {
      if (!editable || !selected || e.target instanceof HTMLInputElement)
        return;

      const { view } = this.props;
      const tab = view.tabs?.[this.state.tab];
      const [path, index] = selected?.split('_');

      const widgets = _.get(tab.panels, path, tab.panels) as ICustomStackProperty[];

      if (!widgets.length)
        return;

      if (!e.shiftKey) {
        if (!await AlertModal.show('Are you sure you want to delete ?'))
          return;
      }

      widgets.splice(+index, 1);
      _api.views.set(view);

      return this.appRef?.current?.focus();
    }

    let tabNumber = parseInt(e.key);
    if (!isNaN(tabNumber)) {
      const { view } = this.props;

      // tabNumber == 1-9 tab #index, 0 - last tab
      tabNumber = tabNumber || view.tabs?.length || 1;

      this.onTabChange(tabNumber - 1);
      e.preventDefault();

      return;
    }

    switch (e.nativeEvent.code) {
      case 'KeyE':
        e.preventDefault();
        this.setState({ editable: !this.state.editable });
        break;

      case 'ArrowLeft':
        this.onTabChange(tab - 1);
        break;

      case 'ArrowRight':
        this.onTabChange(tab + 1);
        break;
    }
  }

  isEditable = (tab: number): boolean => {
    const { view } = this.props;

    if (!view.tabs?.length)
      return false;

    const { layout, panels } = view.tabs[tab] ?? {};
    switch (layout) {
      case TabLayout.Stack:
        return !panels?.length;

      case TabLayout.Screen:
        return false;
    }

    return false;
  }

  onTabChange = (tab: number) => {
    const { view } = this.props;

    tab = Math.min(Math.max(0, tab), view.tabs?.length - 1) || 0;
    const editable = this.isEditable(tab) || this.state.editable;
    this.setState({ tab, vector: null, editable });
  }

  getNewTabName = (): string => {
    let last = 0;
    for (const tab of this.props.view?.tabs ?? []) {
      const match = /^Tab (\d+)$/i.exec(tab.name);
      const index = parseInt(match?.[1]);
      if (!isNaN(index) && index > last)
        last = index;
    }

    return `Tab ${last + 1}`;
  }

  getRandomIcon = (): IconName => {
    const iconPack = Object.keys(fas);
    const keys = iconPack.map(key => fas[key].iconName);
    const index = Math.round(performance.now() * 10000) % keys.length;
    return keys[index];
  }

  onNewTab = (panels: IPanel[] = []) => {
    const { view } = this.props;

    this.createNewTabInternal(panels, null, TabLayout.Empty);
    _api.views.set(view);

    this.onTabChange(view.tabs.length - 1);
    this.setState({ editable: true });
  }

  onNewTabs = (items: ICustomStackListItem[]) => {
    const { view } = this.props;    

    for (const item of items)
      this.createNewTabInternal(item.panels, item.label);

    _api.views.set(view);

    this.onTabChange(view.tabs.length - 1);
    this.setState({ editable: true });
  }

  createNewTabInternal = (panels: IPanel[] = [], name?: string, layout = TabLayout.Stack) => {
    const { view } = this.props;

    view.tabs.push({
      name: name ?? this.getNewTabName(),
      icon: this.getRandomIcon(),
      layout,
      panels
    });
  }

  onDuplicateTab = () => {
    const { view } = this.props;
    const { tab } = this.state;

    const duplicatedTab: ITab = {
      ...view.tabs[tab],
      name: this.getNewTabName(),
    };

    view.tabs.push(duplicatedTab);
    _api.views.set(view);
    this.onTabChange(view.tabs.length - 1);
    this.setState({ editable: true });
  }

  onScreenAdd = (type: ScreenType) => {
    const { view } = this.props;
    let tab = view.tabs[this.state.tab];

    delete tab.panels;

    switch (type) {
      case ScreenType.ColorCorrection:
        tab = { ...tab, name: 'Color Correction', icon: 'palette', layout: TabLayout.Screen, screen: { type: ScreenType.ColorCorrection }, };
        break;

      case ScreenType.LightCards:
        tab = { ...tab, name: 'Light Cards', icon: 'adjust', layout: TabLayout.Screen, screen: { type: ScreenType.LightCards }, };
        break;

      case ScreenType.Playlist:
        tab = { ...tab, name: 'Playlists', icon: 'play', layout: TabLayout.Screen, screen: { type: ScreenType.Playlist }, };
        break;

      case ScreenType.Sequencer:
        tab = { ...tab, name: 'Sequences', icon: 'play', layout: TabLayout.Screen, screen: { type: ScreenType.Sequencer }, };
        break;

      case ScreenType.Snapshot:
        tab = { ...tab, name: 'Snapshot', icon: 'save', layout: TabLayout.Screen, screen: { type: ScreenType.Snapshot }, };
        break;

      case ScreenType.Stack:
        tab = { ...tab, panels: [], layout: TabLayout.Stack };
        break;
    }

    view.tabs[this.state.tab] = tab;
    _api.views.set(view);
  }

  onSetTabsDrawer = (editable: boolean) => {
    this.setState({ editable });
  }

  onPresetChanged = (preset: IPreset) => {
    if (preset.ID !== this.props.preset?.ID)
      _api.presets.select(preset);
  }

  onTabRename = async () => {
    const { view } = this.props;
    const { tab } = this.state;

    const oldName = view.tabs[tab].name;
    const name = await RenameModal.rename(oldName, 'Tab title');
    if (!name || name === oldName)
      return;

    view.tabs[tab].name = name;
    _api.views.set(view);
  }

  onTabDelete = async (tab: number) => {
    if (!await AlertModal.show('Are you sure you want to delete this tab?'))
      return;

    const { view } = this.props;
    if (tab < 0 || tab >= view.tabs.length)
      return;

    view.tabs.splice(tab, 1);
    _api.views.set(view);
    tab = Math.min(tab, view.tabs.length - 1);

    this.setState({ tab });
  }

  onTabIconChange = async () => {
    const { view } = this.props;
    const tab = view.tabs[this.state.tab];
    if (!tab)
      return;

    const newIcon = await IconModal.show(tab.icon);

    if (newIcon) {
      tab.icon = newIcon;
      _api.views.set(view);
    }
  }

  setVectorDrawer = (widget?: ICustomStackProperty) => {
    let vector = null;
    if (widget?.widgets?.length)
      vector = widget;

    this.setState({ vector });
  }

  resetDragState = (state: any = {}) => {
    this.widget = null;

    this.setState({ dragging: null, droppable: null, selected: null, ...state });
  }

  onBeforeCapture = (initial: BeforeCapture) => {
    let { selected } = this.state;
    if (!initial.draggableId.startsWith('REORDER'))
      selected = null;

    this.setState({ dragging: initial.draggableId, selected });
  }

  onDragEnd = (result: DropResult) => {
    const { destination, source, draggableId } = result;

    if (!destination)
      return this.resetDragState();

    const { view } = this.props;

    if (result.type === 'HEADER_TABS') {
      const tab = view.tabs[source.index];

      view.tabs.splice(source.index, 1);
      view.tabs.splice(destination.index, 0, tab);
      _api.views.set(view);

      this.onTabChange(destination.index);

      return this.resetDragState();
    };

    const tab = view.tabs?.[this.state.tab];
    const drop = this.dropElements[destination.droppableId];

    if (['TABS-REORDER', 'LIST-REORDER'].includes(result.type)) {
      let tabs = _.get(tab?.panels, drop.path, tab?.panels);

      if (result.type === 'TABS-REORDER')
        tabs = tabs.tabs;

      if (result.type === 'LIST-REORDER')
        tabs = tabs.items;

      const drag = tabs[source.index];

      tabs.splice(source.index, 1);
      tabs.splice(destination.index, 0, drag);

      return _api.views.set(view);
    };

    if (!drop)
      return this.resetDragState();

    const sourcePath = this.dropElements[source.droppableId]?.path;
    const dropPath = drop?.path;
    const dragType = _.last(draggableId.split('_'));

    const index = source.index;

    if (!tab.panels)
      tab.panels = [];

    const panels = [...tab.panels];
    const dropDestination = _.get(tab?.panels, dropPath, tab?.panels);

    if (!this.widget?.length) {
      this.widget = _.get(tab?.panels, sourcePath, tab?.panels).splice(index, 1);

      if (dropPath === sourcePath && [WidgetTypes.Button, WidgetTypes.ColorPicker, WidgetTypes.MiniColorPicker, WidgetTypes.Toggle].includes(this.widget[0]?.widget as WidgetTypes)) {
        if (destination.droppableId !== source.droppableId && source.index < destination.index)
          destination.index -= 1;
      }
    }

    let el = [...this.widget];

    switch (dragType) {
      case 'PANEL':
        break;

      case 'LIST':
        el = [{ type: IPanelType.List, items: [{ label: 'Item 1', panels }] }];
        dropDestination.splice(0, dropDestination.length);
        break;

      default:
        if (['ROOT', 'LIST'].includes(drop.type))
          el = [{ type: IPanelType.Panel, widgets: el }];
        break;
    }

    dropDestination.splice(destination.index, 0, ...el);

    this.resetDragState({ selected: `${dropPath}_${destination.index}_${_.first(el).property || null}` });
    _api.views.set(view);
  }

  onSelected = (selected: string) => {
    if (!this.state.editable)
      return;

    this.setState({ selected });
  }

  onResetSelection = (e: React.MouseEvent) => {
    this.setState({ selected: null });
  }

  renderTab = () => {
    const { view } = this.props;
    if (!view.tabs || !view.tabs.length)
      return null;

    const index = this.state.tab;
    const tab = view.tabs[index];
    if (!tab)
      return null;

    let visible = true;
    const style: React.CSSProperties = {};
    if (this.state.tab !== index) {
      visible = false;
      style.display = 'none';
    }

    let content = null;
    switch (tab.layout) {
      case TabLayout.Stack:
        content = this.renderStack(tab, visible, index);
        break;

      case TabLayout.Screen:
        content = this.renderScreen(tab, visible);
        break;

      case TabLayout.Empty:
        content = <EmptyTab onAddScreen={this.onScreenAdd} />;
    }

    return (
      <section id="panels" key={index} className={`layout-${tab.layout}`} style={style}>
        {content}
      </section>
    );
  }

  renderStack = (tab: ITab, visible: boolean, index: number) => {
    const { editable, dragging, droppable, selected, hoverTab, vector } = this.state;
    const tabKey = `${tab.name}-${index}`;

    return (
      <div ref={this.panelRef} className="panel" onClick={this.onResetSelection}>
        <Stack panels={tab.panels}
               editable={editable}
               visible={visible}
               dragging={dragging}
               droppable={droppable}
               selected={selected}
               tabKey={tabKey}
               hoverTab={hoverTab}
               vector={vector}
               onSelected={this.onSelected}
               getDroppableId={this.onDropElementAdd}
               onSetVectorDrawer={this.setVectorDrawer} />
      </div>
    );
  }

  renderScreen = (tab: ITab, visible: boolean) => {
    return (
      <div className="panel">
        <Screen screen={tab.screen} visible={visible} />
      </div>
    );
  }

  onUpdateView = (update: boolean) => {
    const { view } = this.props;

    if (update)
      this.setState({ selected: null });

    _api.views.set(view);
  }

  onLockedChange = () => {
    const { lockedUI } = this.props;
    _api.lockUI(!lockedUI);
  }

  checkPassphrase = async (passphrase: string): Promise<boolean> => {
    return await _api.passphrase.login(passphrase);
  }

  renderStatus = () => {
    const { keyCorrect, connected, loading, isOpen, preset } = this.props;

    if (!connected)
      return <ConnectionStatus />;
  
    if (!keyCorrect && !isOpen)
      return <Login onSubmit={this.checkPassphrase} />;

    if (loading) {
      return (
        <div className="fullscreen">
          <div className="app-icon">
            <FontAwesomeIcon icon={['fas', 'sync-alt']} spin />
          </div>
          <div>Loading Preset Data...</div>
        </div>
      );
    }

    if (!preset)
      return <h1 className="fullscreen opaque">No Preset Available</h1>;

    return null;
  }

  render() {
    const { view, preset, presets, lockedUI, undo, redo, signal } = this.props;
    const { tab, editable, selected } = this.state;

    let appClassName = 'tabs-closed ';

    if (editable)
      appClassName = appClassName.replace('closed', 'open');

    if (lockedUI)
      appClassName += 'locked ';

    return (
      <DragDropContext onBeforeCapture={this.onBeforeCapture}
                       onDragEnd={this.onDragEnd}>
        <div id="app"
             className={appClassName}
             tabIndex={1}
             ref={this.appRef}
             onKeyDown={this.onKeyDown} >
          {this.renderStatus()}

          {lockedUI && 
            <div className='locked-layer' />
          }
          <nav id="top-bar">
            <div className="app-icon">
              <img src="/images/favicon/32x32.png" alt="Unreal Engine" />
            </div>
           
            <div className="tab edit-tab mode-toggle">
              <label className="switch toggle-mode">
                <input type="checkbox" checked={!!editable} onChange={() => this.onSetTabsDrawer(!editable)} />
                <span className="slider inline"></span>
              </label>
            </div>

            <Tabs preset={preset}
                  tabs={view.tabs}
                  selected={tab}
                  onChange={this.onTabChange}
                  onNewTab={this.onNewTab}
                  editable={editable}
                  locked={lockedUI}
                  undo={!!undo.length}
                  redo={!!redo.length}
                  onDeleteTab={this.onTabDelete}
                  onLockedChange={this.onLockedChange} />
          </nav>

          {!!preset &&
            <PropertiesDrawer preset={preset}
                              presets={presets}
                              editable={editable}
                              tab={tab}
                              view={view}
                              panels={view.tabs[tab]?.panels || []}
                              selected={selected}
                              onChangeIcon={this.onTabIconChange}
                              onRenameTabModal={this.onTabRename}
                              onDuplicateTab={this.onDuplicateTab}                              
                              onPresetChange={this.onPresetChanged}
                              onUpdateView={this.onUpdateView}
                              onSelected={this.onSelected}
                              onNewTabs={this.onNewTabs}
                              lockWidget={widget => this.widget = widget} />
          }
          {this.renderTab()}
          <SignalIcon signal={signal} />
        </div>
      </DragDropContext>
    );
  }
}