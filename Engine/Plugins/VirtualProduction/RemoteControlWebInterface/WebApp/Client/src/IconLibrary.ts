import { IconPack, library } from '@fortawesome/fontawesome-svg-core';
import { fas } from '@fortawesome/free-solid-svg-icons';
import { far } from '@fortawesome/free-regular-svg-icons';

library.add(fas as IconPack, far as IconPack);