import React from 'react';
import { ReactComponent as Logo } from '../assets/ue_logo.svg';


export class ConnectionStatus extends React.Component {
  render() {
    return (
      <div className="fullscreen">
        <div className="app-icon">
          <Logo className="logo" />
        </div>
        <div>Connecting to Remote Control API...</div>
     </div>
    );
  }
};