import React from 'react';
import { Draggable, Droppable, DraggableProps, DroppableProps, DraggableStateSnapshot } from 'react-beautiful-dnd';
import { Virtuoso, ItemProps } from 'react-virtuoso';
import { IPanel, WidgetTypes } from 'src/shared';


export type DraggableWidgetProps = Partial<DraggableProps> & {
  index: number;
  className?: string;
  children?: any;
  alignRight?: boolean;
  selected?: boolean;

  onPointerChange?:(draggableId: string | null) => void;
  onSelect?: (e?: React.MouseEvent) => void;
};

export class DraggableWidget extends React.Component<DraggableWidgetProps> {

  getStyle = (style: React.CSSProperties, snapshot: DraggableStateSnapshot): React.CSSProperties => {
    if (!snapshot.isDropAnimating) {
      if (!snapshot.isDragging && !!style.transform) {      
        const position = style.transform.replace(/translate|[(|)]/g, '');
        const [left, top] = position.split(', ');
        
        style = { ...style, transform: '', left, top };
      };

      return style;
    }

    return { ...style, transitionDuration: '0.001s' };
  }

  render() {
    const { children, draggableId, isDragDisabled, index, selected } = this.props;

    return (
      <Draggable key={draggableId}
                 draggableId={draggableId} 
                 index={index}
                 isDragDisabled={isDragDisabled}>
        {(provided, snapshot) => {
          let className = `draggable-item ${this.props.className ?? ''} `;
          let dragHandleClass = 'drag-handle ';

          if (!isDragDisabled && !!children)
            className += 'edit ';

          if (snapshot.isDragging)
            className += 'dragging ';

          if (selected)
            className += 'selected ';

          return (
            <div ref={provided.innerRef}
                 className={className}
                 {...provided.draggableProps}
                 style={this.getStyle(provided.draggableProps.style, snapshot)}
                 onClick={e => {
                   if (!e.defaultPrevented)
                     this.props.onSelect?.(e);

                   e.preventDefault();
                   e.stopPropagation();
                 }}>
              <span {...provided.dragHandleProps}
                    onPointerDown={() => this.props.onPointerChange?.(draggableId)}
                    className={dragHandleClass}
                    tabIndex={-1}>
                <img src="/images/GripDotsBlocks.svg" alt="drag" className="grip-handle"/>
              </span>
              {children}
            </div>
          );
        }}
      </Draggable>
    );
  }
};

export type DroppableWidgetProps = Partial<DroppableProps> & {
  className?: string;
  children?: any;
  type?: WidgetTypes;
  style?: React.CSSProperties;
};

export class DroppableWidget extends React.Component<DroppableWidgetProps> {
  
  render() {
    const { children, droppableId, type, style = {}, ...props } = this.props;

    return (
      <Droppable key={droppableId}
                 droppableId={droppableId}
                 {...props}>
        {(provided, snapshot) => {
          let className = `droppable ${this.props.className ?? ''} `;

          if (snapshot.isDraggingOver)
            className += 'dragging ';

          return (
            <div ref={provided.innerRef}
                 className={className}
                 data-type={type}
                 {...provided.droppableProps}
                 style={style}>
              {children}
              {provided.placeholder}
            </div>
          );
        }}
      </Droppable>
    );
  }
}

export type DroppableVirtualListProps = Partial<DroppableProps> & {
  className?: string;
  data: IPanel[];

  itemContent: (index: number, panel: IPanel) => React.ReactElement;
}

export class VirtualList extends React.Component<DroppableVirtualListProps> {

  renderItemPlaceholder = (itemProps: ItemProps & { children: React.ReactNode }) => {
    const { children, ...props } = itemProps;

    return (
      <div {...props} style={{ height: props['data-known-size'] ?? undefined }}>
        {children}
      </div>
    );
  }

  getItemSize = (el: HTMLElement) => {
    const child = el.firstChild as HTMLElement;

    return child?.clientHeight ?? el.clientHeight;
  }
  
  render() {
    const { data, droppableId, className, ...props } = this.props;

    return (
      <Droppable {...props}
                 droppableId={droppableId}
                 key={droppableId}>
        {provided => {
          let className = `droppable ${this.props.className ?? ''} `;

          return <Virtuoso scrollerRef={provided.innerRef}
                           data={data}
                           style={{ height: '100%' }}
                           components={{ Item: this.renderItemPlaceholder }}
                           className={className}
                           itemSize={this.getItemSize}
                           itemContent={this.props.itemContent} />;
        }}
      </Droppable>
    );
  }
}