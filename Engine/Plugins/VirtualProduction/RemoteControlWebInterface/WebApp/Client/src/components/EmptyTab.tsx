import React from 'react';
import { ScreenType } from 'src/shared';


type Props = {
  onAddScreen: (screen: ScreenType) => void;
}

export class EmptyTab extends React.Component<Props> {

  screens = [
    { type: ScreenType.Stack, title: 'Build Your Own UI' },
    { type: ScreenType.ColorCorrection, title: 'Color Correction' },
    { type: ScreenType.Playlist, title: 'Playlist' },
    { type: ScreenType.Snapshot, title: 'Snapshot' },
    { type: ScreenType.Sequencer, title: 'Sequencer' },
  ];

  render() {
    return (
      <div className="panel">
        <div className="empty-tab-container">
          <ul className="screens-list">
            {this.screens.map(({ type, title }) => (
              <li className="screen-type-item" key={type}>
                <button className="btn btn-secondary" onClick={() => this.props.onAddScreen(type)}>{title}</button>
              </li>
            ))}
          </ul>
        </div>
      </div>
    );
  }
};