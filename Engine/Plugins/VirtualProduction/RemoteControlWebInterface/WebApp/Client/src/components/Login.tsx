import React from 'react';
import { ReactComponent as Logo } from '../assets/ue_logo.svg';
import crypto from 'crypto';

type Props = {
  value?: string;
  onSubmit?: (password) => void;
}

type State = {
  value: string;
  error: boolean;
};

export class Login extends React.Component<Props> {
  onClick = () => {
    this.submit();
  }

  submit = async (initialSubmit?: boolean) => {
    const secured = crypto.createHash('md5').update(this.state.value).digest('hex');
    try {
      await this.props.onSubmit(secured);
    } catch (e) {
      const newState = { value: this.state.value, error: !initialSubmit };
      this.setState(newState);
    }
  }

  state: State = {
    value: '',
	error: false
  };

  componentDidMount() {
    document.addEventListener('keydown', this.onKeyPress);
    const { value } = this.props;
    this.setState({ value });
	this.submit(true);
  }

  onChange = (e: React.ChangeEvent<HTMLInputElement>) => {
    const value = e.target.value;
    this.setState({ value });
  }

  componentWillUnmount() {    
    document.removeEventListener('keydown', this.onKeyPress);
  }

  onKeyPress = (e) => {
    if (e.key === 'Enter')
      this.submit();
  }

  render() {
    const { value } = this.props;

    return (
      <div className="fullscreen login-screen">
        <div className='icon-wrapper'>
          <div className="app-icon">
            <Logo className="logo" />
          </div>
          <div>Remote Control Web App</div>
        </div>
        <div className='form'>
          Password
          <input type='password' value={value} onChange={this.onChange}/>
          <button className="btn btn-secondary" onClick={this.onClick}>Submit</button>
          <div className='hint'>Configure the password in: <span className="breadcrumb">Project Settings / Authentication / Password</span></div>
		  
          <label className='login-status' style={{visibility: this.state.error ? 'visible' : 'hidden' }}>Incorrect Password</label>
		  
        </div>
     </div>
    );
  }
};