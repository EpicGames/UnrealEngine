import { FontAwesomeIcon } from '@fortawesome/react-fontawesome';
import React from 'react';
import { IPreset } from 'src/shared';
import { Search } from '../controls';
import { _api } from '../../reducers';
import _ from 'lodash';
import { IconProp } from '@fortawesome/fontawesome-svg-core';


type Props = {
  preset: IPreset;
  pressets: IPreset[];

  onPresetChange: (preset: IPreset) => void;
  onSearch: (value: string) => void;
}

type State = {
  spin: string;
}

export class Pressets extends React.Component<Props, State> {

  state: State = {
    spin: null,
  }

  onFavoriteToggled = (e: React.MouseEvent, preset: IPreset) => {
    const { spin } = this.state;
    e.stopPropagation();

    if (spin !== preset.ID)
      this.setState({ spin: preset.ID });

    this.onDebouncedFavoriteClick(preset);
  }

  onFavoritePreset = async (preset: IPreset) => {
    if (!preset.IsFavorite)
      this.props.onPresetChange(preset);

    await _api.presets.favorite(preset.ID, !preset.IsFavorite);
    this.setState({ spin: null });
  }

  onDebouncedFavoriteClick = _.debounce(this.onFavoritePreset, 100);

  render() {
    const { preset, pressets } = this.props;
    const { spin } = this.state;

    return (
      <div className="presets-tab">
        <Search placeholder="Search Presets" onSearch={this.props.onSearch} />
        <div className="presets-wrapper">
          {pressets.map(p => {
            let className = 'btn preset-btn ';
            let spinIconClassName = 'spin-icon ';
            let icon: IconProp = ['far', 'star'];

            if (p.IsFavorite)
              icon = ['fas', 'star'];

            if (p.ID === preset?.ID) {
              className += 'active';
            }

            if (spin === p.ID)
              spinIconClassName += 'visible';

            return (
              <div key={p?.ID} className={className} onClick={() => this.props.onPresetChange(p)}>
                <FontAwesomeIcon className={spinIconClassName} icon={['fas', 'sync']} spin />
                <FontAwesomeIcon className='favorite-icon' icon={icon} onClick={e => this.onFavoriteToggled(e, p)} />
                {p?.Name}
                <div className="item-icon">
                  <FontAwesomeIcon icon={['fas', 'external-link-square-alt']} />
                </div>
              </div>
            );
          })}
        </div>
      </div>
    );
  }
};
