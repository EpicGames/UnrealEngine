import React from 'react';
import { Search } from '../controls';
import { Droppable, Draggable, DraggableStateSnapshot } from 'react-beautiful-dnd';
import { IExposedFunction, IExposedProperty, IGroup, IPreset } from 'src/shared';
import { FontAwesomeIcon } from '@fortawesome/react-fontawesome';
import { IconProp } from '@fortawesome/fontawesome-svg-core';
import _ from 'lodash';


type Props = {
  groups: IGroup[];
  preset: IPreset;

  renderDraggableItem: (element: IExposedProperty & IExposedFunction, index: number) => void;
  onWidgetsLock: (widgets: (IExposedProperty | IExposedFunction)[]) => void;
  onSerach: (value: string) => void;
}

type State = {
  collapsed: Record<string, boolean>;
}

export class Properties extends React.Component<Props, State> {

  state: State = {
    collapsed: {}
  }

  componentDidUpdate(prevProps: Props) {
    const { preset } = this.props;

    if (preset?.Name !== prevProps.preset?.Name)
      this.setState({ collapsed: {} });
  }

  onToggleGroup = (key: string) => {
    const { collapsed } = this.state;

    this.setState({ collapsed: { ...collapsed, [key]: !collapsed[key] }  });
  }

  getStyle = (style: React.CSSProperties, snapshot: DraggableStateSnapshot): React.CSSProperties => {
    if (!snapshot.isDragging)
      return { ...style, transform: 'none' };

    if (!snapshot.isDropAnimating)
      return style;

    return { ...style, transitionDuration: '0.001s' };
  }

  renderGroup = (group: IGroup, index: number) => {
    const { collapsed } = this.state;
    const exposed = [...group.ExposedProperties, ...group.ExposedFunctions];
    
    if (!exposed.length)
      return null;

    let icon: IconProp = ['fas', 'caret-down'];
    let className = 'items-list ';

    if (collapsed[group.Name])
      icon = ['fas', 'caret-right'];
    else
      className += 'expanded';

    return (
      <li key={group.Name}>
        <Draggable draggableId={`${group.Name}_WIDGET`} index={index}>
          {(provided, snapshot) => {

            if (snapshot.isDragging)
              this.props.onWidgetsLock(_.filter(exposed, p => !!p.Metadata?.Widget));

            return (
              <>
                {snapshot.isDragging && (
                  <div className="list-title">
                    {group.Name}
                    <FontAwesomeIcon icon={icon} />
                  </div>
                )}
                <div {...provided.draggableProps}
                     {...provided.dragHandleProps}
                     className="list-title"
                     onClick={this.onToggleGroup.bind(this, group.Name)}
                     ref={provided.innerRef}
                     style={this.getStyle(provided.draggableProps.style, snapshot)}>
                  {group.Name}
                  <FontAwesomeIcon icon={icon} />
                </div>
              </>
            );
          }}
        </Draggable>
        <Droppable key={group.Name}
                   droppableId={`ADDWIDGET_${group.Name}`}
                   isDropDisabled>
          {(provided) => {
            return (
              <ul ref={provided.innerRef}
                   className={className}
                {...provided.droppableProps}>
                {!collapsed[group.Name] && exposed.map(this.props.renderDraggableItem)}
                <div style={{ display: 'none' }}>{provided.placeholder}</div>
              </ul>
            );
          }}
        </Droppable>
      </li>
    );
  }

  render() {
    const { groups } = this.props;

    return (
      <div className="properties-tab">
        <Search placeholder="Search Properties" onSearch={this.props.onSerach} />
        <div className="nav-list-container">
          <Droppable droppableId="properties-droppable">
            {provided => (
              <ul ref={provided.innerRef}
                  className="nav-list"
                {...provided.droppableProps}>
                {groups.map(this.renderGroup)}
              </ul>
            )}
          </Droppable>
        </div>
      </div>
    );
  }
};
