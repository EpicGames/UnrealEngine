import React from 'react';


type Props = {
  onChangeIcon: () => void;
  onRenameTabModal: () => void;
  onDuplicateTab: () => void;
}

export class Tab extends React.Component<Props> {
  render() {
    return (
      <div className="tabs-tab">
        <div className="button-list">
          <button tabIndex={-1} className="btn btn-secondary" onClick={this.props.onChangeIcon}>Change tab icon</button>
          <button tabIndex={-1} className="btn btn-secondary" onClick={this.props.onRenameTabModal}>Rename Tab</button>
          <button tabIndex={-1} className="btn btn-secondary" onClick={this.props.onDuplicateTab}>Duplicate Tab</button>
        </div>
      </div>
    );
  }
};
