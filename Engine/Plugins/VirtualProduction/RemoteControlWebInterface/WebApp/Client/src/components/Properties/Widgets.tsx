import React from 'react';
import { Search } from '../controls';
import { Droppable, Draggable, DraggableStateSnapshot } from 'react-beautiful-dnd';
import { FontAwesomeIcon } from '@fortawesome/react-fontawesome';
import { IPanelType, WidgetTypes, ICustomStackProperty } from 'src/shared';
import { DropWidgetsType } from '../';


type Props = {
  widgets: DropWidgetsType[];

  renderWidget: (widget: ICustomStackProperty) => React.ReactNode;
  onSearch: (value: string) => void;
}

export class Widgets extends React.Component<Props> {


  getStyle = (style: React.CSSProperties, snapshot: DraggableStateSnapshot): React.CSSProperties => {
    if (!snapshot.isDragging)
      return { ...style, transform: 'none' };

    if (!snapshot.isDropAnimating)
      return { ...style, height: 'auto', width: 'auto', minWidth: '350px' };

    return { ...style, height: 'auto', width: 'auto', minWidth: '350px', transitionDuration: '0.001s' };
  }

  render() {
    const { widgets } = this.props;

    return (
      <div className="widgets-tab">
        <Search placeholder="Search Widgets" onSearch={this.props.onSearch} />
        <div className="layout-widgets">
          {widgets.map(w => (
            <Droppable droppableId={w.id} key={w.id} isDropDisabled>
              {(provided) => (
                <div {...provided.droppableProps} 
                     ref={provided.innerRef}>
                  <Draggable key={w.id} 
                             draggableId={w.id}
                             index={0}>
                    {(provided, snapshot) => {
                      let className = 'layout-panel layout-widget draggable-widget';
                      let itemIcon: JSX.Element = null;
                      let widget = null;
                      let dragging = null;

                      switch (w.type) {
                        case 'PANEL':
                          itemIcon = <FontAwesomeIcon icon={['fas', 'columns']} />;
                          widget = { type: IPanelType.Panel, widgets: [] };
                          break;

                        case 'LIST':
                          itemIcon = <FontAwesomeIcon icon={['fas', 'list']} />;
                          widget = { type: IPanelType.List, items: [{ label: 'Item 1', panels: [] }] };
                          break;

                        case WidgetTypes.Tabs:
                          itemIcon = <FontAwesomeIcon icon={['fas', 'table']} />;
                          widget = { widget: WidgetTypes.Tabs, tabs: [{ label: 'Tab 1', widgets: [] }] };
                          break;

                        case WidgetTypes.Label:
                          itemIcon = <FontAwesomeIcon icon={['fas', 'font']} />;
                          widget = { widget: WidgetTypes.Label, label: 'New label' };
                          break;

                        case WidgetTypes.Spacer:
                          itemIcon = <FontAwesomeIcon icon={['fas', 'square']} />;
                          widget = { widget: WidgetTypes.Spacer, label: '' };
                          break;

                        case WidgetTypes.Dial:
                          itemIcon = <div className="item-icon"><img src="/images/icons/Dial.svg" alt="Slider" /></div>;
                          widget = { widget: WidgetTypes.Dial, property: null, propertyType: null };
                          break;

                        case WidgetTypes.Slider:
                          itemIcon = <div className="item-icon"><img src="/images/icons/Sliders.svg" alt="Slider" /></div>;
                          widget = { widget: WidgetTypes.Slider, property: null, propertyType: null };
                          break;

                        case WidgetTypes.ColorPicker:
                          itemIcon = <div className="item-icon">Color Picker</div>;
                          widget = { widget: WidgetTypes.ColorPicker, property: null, propertyType: null };
                          break;

                        case WidgetTypes.MiniColorPicker:
                          itemIcon = <div className="item-icon">Mini Color Picker</div>;
                          widget = { widget: WidgetTypes.MiniColorPicker, property: null, propertyType: null };
                          break;

                        case WidgetTypes.ColorPickerList:
                          itemIcon = <div className="item-icon">Color Picker List</div>;
                          widget = { widget: WidgetTypes.ColorPickerList, items: [] };
                          break;

                        case WidgetTypes.Toggle:
                          itemIcon = <div className="item-icon">Toggle</div>;
                          widget = { widget: WidgetTypes.Toggle, property: null, propertyType: null };
                          break;

                        case WidgetTypes.Joystick:
                          itemIcon = <div className="item-icon"><img src="/images/icons/Joystick.svg" alt="Slider" /></div>;
                          widget = { widget: WidgetTypes.Joystick, property: null, propertyType: null };
                          break;

                        case WidgetTypes.Button:
                          itemIcon = <div className="item-icon">Button</div>;
                          widget = { widget: WidgetTypes.Button, property: null, propertyType: null };
                          break;

                        case WidgetTypes.Text:
                          itemIcon = <div className="item-icon">Text</div>;
                          widget = { widget: WidgetTypes.Text, property: null, propertyType: null };
                          break;

                        case WidgetTypes.Dropdown:
                          itemIcon = <div className="item-icon">Dropdown</div>;
                          widget = { widget: WidgetTypes.Dropdown, property: null, propertyType: null };
                          break;

                        case WidgetTypes.Vector:
                          itemIcon = <div className="item-icon">Vector</div>;
                          widget = { widget: WidgetTypes.Vector, property: null, propertyType: null };
                          break;
                      }
 
                      if (snapshot.isDragging) {
                        className += ' dragging';
                        dragging = this.props.renderWidget(widget);

                        if (!dragging)
                          dragging = (
                            <>
                              <img src="/images/GripDotsBlocks.svg" alt="drag" className="grip-handle" />
                              {w.label}
                              <div className="item-icon">{itemIcon}</div>
                            </>
                          );
                      }

                      return (
                        <>
                          {snapshot.isDragging &&
                            <div className="draggable-widget">
                              <img src="/images/GripDotsBlocks.svg" alt="drag" className="grip-handle" />
                              <div className="item-label">{w.label}</div>
                              <div className="item-icon">{itemIcon}</div>
                            </div>
                          }
                          <div {...provided.draggableProps}
                               {...provided.dragHandleProps}
                               className={className}
                               ref={provided.innerRef}
                               style={this.getStyle(provided.draggableProps.style, snapshot)}>
                            {snapshot.isDragging ? dragging : (
                              <>
                                <img src="/images/GripDotsBlocks.svg" alt="drag" className="grip-handle" />
                                <span className="item-label">{w.label}</span>
                                <div className="item-icon">{itemIcon}</div>
                              </>
                            )}
                          </div>
                        </>
                      );
                    }}
                  </Draggable>
                  <div style={{ display: 'none' }}>{provided.placeholder}</div>
                </div>
              )}
            </Droppable>
          ))}
        </div>
      </div>
    );
  }
};
