import React, { Component } from 'react';
import { _api } from 'src/reducers';
import { IPreset, IGroup, IExposedProperty, WidgetTypes, ICustomStackProperty, IPanel, ICustomStackTabs, IExposedFunction, 
        PropertyType, IView, ITab, TabLayout, ICustomStackWidget, WidgetType, IColorPickerList, ICustomStackListItem } from 'src/shared';
import { Draggable, Droppable, DraggableStateSnapshot } from 'react-beautiful-dnd';
import { FontAwesomeIcon } from '@fortawesome/react-fontawesome';
import { AlertModal } from '.';
import { SlidersWidget, SliderWidget, DropdownWidget, ImageSelectorWidget, ToggleWidget, VectorWidget, AssetWidget } from './Widgets';
import * as Tabs from './Properties';
import { ColorPicker, Button, Text, Dial, JoysticksWrapper, DialsWrapper } from './controls';
import { WidgetUtilities } from 'src/utilities';
import _ from 'lodash';


export type DropWidgetsType = {
  id: string;
  label: string;
  type: string;
  propertyType?: PropertyType;
}

type State = {
  filteredGroups: IGroup[];
  filteredPresets?: IPreset[];
  filteredWidgets: DropWidgetsType[],

  search: string;

  editTab: boolean;
  tab: TabType;

  metadata: Record<string, string>;

  widget: ICustomStackProperty | ICustomStackTabs | IPanel | IColorPickerList;
  usedProperties: Record<string, boolean>;
};

type Props = {
  selected: string;
  panels: IPanel[];
  presets: IPreset[];
  preset: IPreset;
  view: IView;
  tab: number;
  editable: boolean;

  lockWidget: (widget: any) => void;

  onChangeIcon: () => void;
  onRenameTabModal: () => void;
  onDuplicateTab: () => void;
  onUpdateView: (update?: boolean) => void;
  onPresetChange: (preset: IPreset) => void;
  onSelected: (selected: string) => void;
  onNewTabs: (items: ICustomStackListItem[]) => void;
};

enum TabType {
  Widgets     = 'WIDGETS',
  Properties  = 'PROPERTIES',
  Tab         = 'TAB',
  Presets     = 'PRESETS',
}

const tabs = [
  { title: 'Presets',    key: TabType.Presets },
  { title: 'Tab',        key: TabType.Tab },
  { title: 'Widgets',    key: TabType.Widgets },
  { title: 'Properties', key: TabType.Properties },
];

const widgets = [
  { id: 'PANEL',                                 type: 'PANEL',                     label: 'Panel' },
  { id: 'LIST',                                  type: 'LIST',                      label: 'List' },
  { id: WidgetTypes.Tabs,                        type: WidgetTypes.Tabs,            label: 'Tabs' },
  { id: WidgetTypes.Label,                       type: WidgetTypes.Label,           label: 'Label' },
  { id: WidgetTypes.Spacer,                      type: WidgetTypes.Spacer,          label: 'Spacer' },

  { id: WidgetTypes.Dial,                        type: WidgetTypes.Dial,            label: 'Dial',              propertyType: PropertyType.Float },
  { id: `${WidgetTypes.Slider}_WIDGET`,          type: WidgetTypes.Slider,          label: 'Slider',            propertyType: PropertyType.Float },
  { id: WidgetTypes.ColorPicker,                 type: WidgetTypes.ColorPicker,     label: 'Color Picker',      propertyType: PropertyType.LinearColor },
  { id: WidgetTypes.MiniColorPicker,             type: WidgetTypes.MiniColorPicker, label: 'MiniColor Picker',  propertyType: PropertyType.LinearColor },
  { id: `${WidgetTypes.ColorPickerList}_WIDGET`, type: WidgetTypes.ColorPickerList, label: 'Color Picker List', propertyType: PropertyType.LinearColor },
  { id: WidgetTypes.Toggle,                      type: WidgetTypes.Toggle,          label: 'Toggle',            propertyType: PropertyType.Boolean },
  { id: `${WidgetTypes.Joystick}_WIDGET`,        type: WidgetTypes.Joystick,        label: 'Joystick',          propertyType: PropertyType.Vector },
  { id: WidgetTypes.Button,                      type: WidgetTypes.Button,          label: 'Button',            propertyType: PropertyType.Function },
  { id: `${WidgetTypes.Text}_WIDGET`,            type: WidgetTypes.Text,            label: 'Text',              propertyType: PropertyType.Text },
  { id: WidgetTypes.Dropdown,                    type: WidgetTypes.Dropdown,        label: 'Dropdown',          propertyType: PropertyType.Text },
  { id: `${WidgetTypes.Vector}_WIDGET`,          type: WidgetTypes.Vector,          label: 'Vector',            propertyType: PropertyType.Vector },
];

export class PropertiesDrawer extends Component<Props, State> {

  state: State = {
    filteredGroups: [...this.props.preset.Groups],
    filteredWidgets: [..._.sortBy(widgets, w => w.label)],
    filteredPresets: [...this.props.presets],
    search: '',
    editTab: false,
    tab: TabType.Presets,

    widget: null,
    metadata: {},
    usedProperties: {},
  };

  componentDidUpdate(prevProps: Props, prevState: State) {
    const { editable, selected, panels, preset, view, tab } = this.props;

    if (!editable)
      return;

    if (editable !== prevProps.editable || view !== prevProps.view || tab !== prevProps.tab)
      this.buildPropertyList(view?.tabs?.[tab]);

    if (view !== prevProps.view || selected && (selected !== prevProps.selected || !_.isEqual(panels, prevProps.panels))) {
      const [path, index, property] = selected?.split('_') ?? [];

      let widget;
      if (!index)
        widget = this.getWidgetFromElement(preset.Exposed[property] as any); 
      else
        widget = _.get(panels, path, panels)[+index];

      this.setState({ widget, metadata: {} });
    }

    if (preset !== prevProps.preset){
      const { search } = this.state;
      this.onPropertiesFilter(search);
      this.onPresetFilter(search);
    }

    if (this.state.tab !== prevState.tab)
      this.resetFilter();
  }

  buildPropertyList = (tab: ITab) => {
    let usedPropArray = [];
    if (tab?.layout === TabLayout.Stack)
      usedPropArray = _.compact(this.getProperties(tab.panels));

    const usedProperties = _.uniq(usedPropArray).reduce((result, item) => {
      result[item] = true;
      return result;
    }, {});

    this.setState({ usedProperties });
  }

  resetFilter = () => {
    this.setState({
      filteredGroups: [...this.props.preset.Groups],
      filteredWidgets: [..._.sortBy(widgets, w => w.label)],
      filteredPresets: [...this.props.presets],
    });
  }

  getProperties = (items: any[]) => {
    if (!items)
      return [];

    const properties = [];
    for (const item of items) {
      switch (item.type) {
        case 'LIST':
          for (const listItem of item.items)
            properties.push(...this.getProperties(listItem.panels));
          break;

        case 'PANEL':
          properties.push(...this.getProperties(item.widgets));
          break;

        default:
          properties.push(...this.getPropertiesofWidget(item));
          break;
      }
    }

    return properties;
  }

  getPropertiesofWidget = (item: any) => {
    const properties = [];
    switch (item.widget) {
      case WidgetTypes.Tabs:
        for (const tab of item.tabs)
          properties.push(...this.getProperties(tab.widgets));
        break;

      case WidgetTypes.ColorPickerList:
        for (const picker of item.items)
          properties.push(picker.property);
        break;

      default:
        properties.push(item.property);
        break;
    }

    return properties;
  }

  getStyle = (style: React.CSSProperties, snapshot: DraggableStateSnapshot): React.CSSProperties => {
    if (!snapshot.isDragging)
      return { ...style, transform: 'none' };

    if (!snapshot.isDropAnimating)
      return { ...style, height: 'auto', width: 'auto', minWidth: '350px' };

    return { ...style, height: 'auto', width: 'auto', minWidth: '350px', transitionDuration: '0.001s' };
  }

  onPropertiesFilter = (search: string) => {
    const { preset } = this.props;

    let filteredGroups = preset.Groups;
    if (search)
      filteredGroups = _.map(preset.Groups, group => this.filterGroup(group, search));

    this.setState({ filteredGroups, search });
  }

  onWidgetsFilter = (search: string) => {
    const filteredWidgets = _.sortBy(widgets.filter(widget => widget.label.toLowerCase().slice(0, search.length) === search), w => w.label);

    this.setState({ filteredWidgets });
  }

  filterGroup = (group: IGroup, search: string): IGroup => ({
    ...group,
    ExposedProperties: _.filter(group.ExposedProperties, prop => prop.DisplayName.toLowerCase().indexOf(search) !== -1),
    ExposedFunctions: _.filter(group.ExposedFunctions, func => func.DisplayName.toLowerCase().indexOf(search) !== -1),
  });

  onPresetFilter = (search: string) => {
    const { presets } = this.props;

    let filteredPresets = presets;
    if (search)
      filteredPresets = presets.filter(preset => preset.Name.toLowerCase().indexOf(search) !== -1);

    this.setState({ filteredPresets, search });
  }

  onViewUpdatedDebounced = _.debounce(this.props.onUpdateView, 400);

  getWidgetFromElement = (element: IExposedProperty & IExposedFunction): ICustomStackProperty => {
    if (!element)
      return null;

    return {
      property: element.ID,
      propertyType: element?.Type,
      widget: element.Metadata?.Widget,
    };
  };

  onWidgetRemove = async (title: string) => {
    const { selected, panels } = this.props;
    const [path, index] = selected.split('_');

    if (!await AlertModal.show(title))
      return;

    const widgets = _.get(panels, path, panels);
    widgets.splice(+index, 1);

    this.props.onUpdateView(true);
  }

  onWidgetTabRemove = async (i: number) => {
    const { selected, panels } = this.props;
    const [path, index] = selected.split('_');

    if (!await AlertModal.show('Are you sure you want to delete this tab?'))
      return;

    const widgets = _.get(panels, `${path}[${index}]tabs`, panels);
    widgets.splice(i, 1);

    this.props.onUpdateView();
  }

  onListItemRemove = async (i: number) => {
    const { selected, panels } = this.props;
    const [path, index] = selected.split('_');

    if (!await AlertModal.show('Are you sure you want to delete this item?'))
      return;

    const items = _.get(panels, `${path}[${index}]items`, panels); 
    items.splice(i, 1);

    this.props.onUpdateView();
  }

  onChange = (property: string, value: any) => {
    const { selected, panels } = this.props;
    const widget = { ...this.state.widget };
    const [path, index] = selected.split('_');

    _.set(widget, property, value);
    this.setState({ widget } as any);
    let widgets = _.get(panels, path, panels);

    _.set(widgets, `${index}.${property}`, value);
    this.onViewUpdatedDebounced();
  }

  renderWidget = (widget: ICustomStackProperty) => {
    if (!widget)
      return null;

    this.props.lockWidget([widget]);

    switch (widget.widget) {
      case WidgetTypes.Sliders:
        return <SlidersWidget widget={widget} />;

      case WidgetTypes.Slider:
        return <SliderWidget />;

      case WidgetTypes.Dropdown:
        return <DropdownWidget />;

      case WidgetTypes.ImageSelector:
        return <ImageSelectorWidget widget={widget} />;

      case WidgetTypes.Toggle:
        return <ToggleWidget widget={widget} />;

      case WidgetTypes.Label:
        return <div className="label">Label</div>;

      case WidgetTypes.Text:
        return (
          <div className="slider-row text-row">
            <div className="title">Text</div>
            <Text value="Text" />
            <FontAwesomeIcon icon={['fas', 'undo']} />
          </div>
        );

      case WidgetTypes.Vector:
        return <VectorWidget widget={widget} />;

      case WidgetTypes.Button:
        return (
          <div className="btn-wrapper">
            <Button />
          </div>
        );

      case WidgetTypes.ColorPicker:
      case WidgetTypes.MiniColorPicker:
        return <ColorPicker widget={widget.widget} />;

      case WidgetTypes.Joystick:
        return <JoysticksWrapper type={PropertyType.Vector} />;

      case WidgetTypes.Dial:
        return <Dial />;

      case WidgetTypes.Dials:
        return <DialsWrapper type={widget.propertyType} />;

      case WidgetTypes.Asset:
        return <AssetWidget />;

      default:
        return null;
    }
  }

  renderTabsEdit = () => {
    const widget = this.state.widget as ICustomStackTabs;

    return (
      <>
        <Droppable droppableId={widget.id} type="TABS-REORDER">
          {provided => (
            <div className="properties-list"
                 ref={provided.innerRef}
                 {...provided.droppableProps}>
              {widget.tabs?.map((widget, i) => (
                <Draggable key={widget.id} draggableId={`REORDER_${widget.id}`} index={i}>
                  {provided => (
                    <div ref={provided.innerRef}
                         {...provided.draggableProps}
                         className="properties-field line">
                      <span {...provided.dragHandleProps}
                            className="drag-handle"
                            tabIndex={-1}>
                        <img src="/images/GripDotsBlocks.svg" alt="drag" className="grip-handle" />
                      </span>
                      <input type="text"
                             value={widget.label ?? ''}
                             onChange={e => this.onChange(`tabs[${i}]label`, e.target.value)} />
                      <FontAwesomeIcon className="icon" icon={['fas', 'times']} onClick={() => this.onWidgetTabRemove(i)} />
                    </div>
                  )}
                </Draggable>
              ))}
              {provided.placeholder}
            </div>
          )}
        </Droppable>
        <div className="btn-container">
          <button className="btn btn-primary"
                  onClick={() => this.onChange(`tabs[${widget.tabs.length}]`, { label: `Tab ${widget.tabs.length + 1}`, widgets: [] })}>
            Add new tab
            </button>
        </div>
        {this.renderDelete('Tabs')}
      </>
    );
  }

  renderListsEdit = () => {
    const widget = this.state.widget as IPanel;

    return (
      <>
      <Droppable droppableId={widget.id} type="LIST-REORDER">
          {provided => (
            <div className="properties-list"
                 ref={provided.innerRef}
                 {...provided.droppableProps}>
              {widget.items?.map((item, i) => (
                <Draggable draggableId={`REORDER_${item.id}`} index={i} key={item.id}>
                  {provided => (
                    <div ref={provided.innerRef}
                         {...provided.draggableProps}  
                         className="properties-field line">
                      <span {...provided.dragHandleProps}
                            className="drag-handle"
                            tabIndex={-1}>
                        <img src="/images/GripDotsBlocks.svg" alt="drag" className="grip-handle" />
                      </span>
                      <input type="text"
                             value={item.label ?? ''}
                             onChange={e => this.onChange(`items[${i}]label`, e.target.value)} />
                      {widget.items?.length > 1 && <FontAwesomeIcon className="icon" icon={['fas', 'times']} onClick={this.onListItemRemove.bind(this, i)} />}
                    </div>
                  )}
                </Draggable>
              ))}
              {provided.placeholder}
            </div>
          )}
      </Droppable>
        <div className="btn-container">
          <button className="btn btn-primary"
                  onClick={this.onChange.bind(this, `items[${widget.items.length}]`, { label: `Item ${widget.items.length + 1}`, panels: [] })}>
            Add new item
            </button>
        </div>
        {this.renderDelete('List')}
        <div className="btn-container">
          <button className="btn btn-secondary" onClick={() => this.props.onNewTabs(widget.items)}>
            Create Tabs from List Item
          </button>
        </div>
      </>
    );
  }

  renderColorPickersListEdit = () => {
    return this.renderDelete('Color Picker List');
  }

  renderColorPickerEdit = (available: boolean) => {
    const { preset } = this.props;
    const colorPicker = this.state.widget as ICustomStackProperty;

    const exposed = preset?.Exposed[colorPicker?.property] as IExposedProperty;
    let checked = false;

    if (exposed?.Metadata?.Alpha === '1')
      checked = true;

    return (
      <>
        {available &&
          <div className="properties-list">
            {this.renderCompatibleWidgets()}
            {this.renderPropertyLabel()}
          {colorPicker.propertyType !== PropertyType.Vector4 &&
            <div className="properties-field">
              <label>Show Alpha</label>
              <input className="checkbox" type="checkbox" onChange={e => this.onMetadataChange(colorPicker, 'Alpha', e.target.checked ? '1' : '0')} checked={checked} />
            </div>}
          </div>
        }
        {this.renderDelete()}
      </>
    );
  }

  renderPanelEdit = () => {
    const panel = this.state.widget as IPanel;

    return (
      <div className="properties-list">
        <div className="properties-field">
          <label>Panel Title</label>
          <input type="text"
                value={panel?.title ?? ''}
                onChange={e => this.onChange('title', e.target.value)}
          />
        </div>
        <div className="properties-field">
          <label>Is template</label>
          <input className="checkbox" type="checkbox" checked={!!panel?.isTemplate} onChange={e => this.onChange('isTemplate', e.target.checked)} />
        </div>
        {this.renderDelete('Panel')}
      </div>
    );
  }

  onMetadataChange = (widget: Partial<ICustomStackProperty>, key: string, value: string) => {
    const metadata = { ...this.state.metadata };
    metadata[key] = value;

    this.setState({ metadata });
    _api.payload.metadata(widget.property, key, value);
  }

  onDiscardStateMetadata = (key: string) => {
    const metadata = { ...this.state.metadata };
    delete metadata[key];
    this.setState({ metadata });
  }

  onVectorModeChange = (widget: ICustomStackProperty, value: WidgetTypes.Joystick | WidgetTypes.Sliders | WidgetTypes.Dial | string, initial?: string[]) => {
    const { selected, panels } = this.props;
    const [path, index] = selected.split('_');

    const w = _.get(panels, `${path}.${index}`, panels);
    let widgets = _.xor(w?.widgets || [], [value]);

    if (initial) {
      const diff = _.compact(initial.map(key => widgets.find(el => el === key)));

      if (!diff.length)
        widgets.push(value);
    }

    w.widgets = widgets;
    widget.widgets = widgets;
    
    this.props.onUpdateView();
  }

  renderPropertyLabel = () => {
    const { preset } = this.props;
    const widget = this.state.widget as ICustomStackProperty;

    let value = '', placeholder;
    const exposed = preset?.Exposed[widget?.property] as IExposedProperty;
    if (exposed) {
      placeholder = exposed.DisplayName;
      value = this.state.metadata.Description || exposed.Metadata?.Description || '';
    }

    return (
      <>
        {/* {exposed &&
          <div className="properties-title">{exposed.OwnerObjects?.[0]?.Name} {exposed.UnderlyingProperty?.Name}</div>
        } */}
        <div className="properties-field">
          <label>Label</label>
          <input type="text"
                  placeholder={placeholder}
                  value={value}
                  onChange={e => this.onMetadataChange(widget, 'Description', e.target.value)}
                  onBlur={() => this.onDiscardStateMetadata('Description')} />
        </div>
      </>
    );
  }

  renderMinMax = () => {
    const { preset } = this.props;
    const widget = this.state.widget as ICustomStackProperty;

    switch (widget?.widget) {
      case WidgetTypes.Dial:
      case WidgetTypes.Dials:
      case WidgetTypes.Slider:
      case WidgetTypes.Joystick:
      case WidgetTypes.ScaleSlider:
      case WidgetTypes.Sliders:
      case WidgetTypes.Vector:
        const range = WidgetUtilities.getMinMax(preset, widget?.property);
        const { Min, Max } = this.state.metadata;

        return (
          <>
            <div className="properties-field">
              <label>Minimum</label>
              <input type="number"
                     value={Min ?? range?.min ?? ''}
                     onChange={e => this.onMetadataChange(widget, 'Min', e.target.value)}
                     onBlur={() => this.onDiscardStateMetadata('Min')} />
            </div>
            <div className="properties-field">
              <label>Maximum</label>
              <input type="number"
                     value={Max ?? range?.max ?? ''}
                     onChange={e => this.onMetadataChange(widget, 'Max', e.target.value)}
                     onBlur={() => this.onDiscardStateMetadata('Max')} />
            </div>
          </>
        );
    }

    return null;
  }

  renderDelete = (title?: string) => {
    return (
      <div key="delete" className="btn-container">
        <button className="btn btn-danger" onClick={() => this.onWidgetRemove('Are you sure you want to delete?')}>Delete {title ?? 'Widget'}</button>
      </div>
    );
  }

  renderLabelEdit = () => {
    const widget = this.state.widget as ICustomStackProperty;

    return (
      <>
        <div className="properties-list">
          <div className="properties-field">
            <label>Label</label>
            <input value={widget.label} onChange={e => this.onChange('label', e.target.value)} />
          </div>
        </div>
        {this.renderDelete()}
      </>
    );
  }

  renderSpacerEdit = () => {
    const widget = this.state.widget as ICustomStackProperty;

    return (
      <>
        <div className="properties-list">
          <div className="properties-field">
            <label>Rows</label>
            <input type="number"
                   value={String(widget?.spaces || 1) ?? ''}
                   onChange={e => this.onChange('spaces', +e.target.value)}
            />
          </div>
        </div>
        {this.renderDelete()}
      </>
    );
  }

  renderDialEdit = (available: boolean) => {
    return (
      <>
        {available &&
          <div className="properties-list">
            {this.renderCompatibleWidgets()}
            {this.renderPropertyLabel()}
            {this.renderMinMax()}
          </div>
        }
        {this.renderDelete()}
      </>
    );
  }

  renderVectorEdit = (available: boolean) => {
    const { preset } = this.props;
    const widget = this.state.widget as ICustomStackProperty;
    const vectorModes = widget?.widgets || [];

    const exposed = preset?.Exposed[widget?.property] as IExposedProperty;
    let checked = false;

    if (exposed?.Metadata?.Proportionally === '1')
      checked = true;

    return (
      <>
        {available &&
          <div className="properties-list">
            {this.renderCompatibleWidgets()}
            {this.renderPropertyLabel()}
            {widget.propertyType !== PropertyType.Rotator &&
              this.renderMinMax()
            }
            <div className="properties-field">
              <label>Modes</label>
              <div className="btn-group vector-btn-group">
                {widget.propertyType !== PropertyType.Rotator &&
                  <button className={`btn ${vectorModes.includes(WidgetTypes.Joystick) ? 'btn-primary' : 'btn-secondary'}`}
                          onClick={() => this.onVectorModeChange(widget, WidgetTypes.Joystick)}>Joystick</button>
                }
                <button className={`btn ${vectorModes.includes(WidgetTypes.Dial) ? 'btn-primary' : 'btn-secondary'}`}
                        onClick={() => this.onVectorModeChange(widget, WidgetTypes.Dial)}>Dial</button>
                <button className={`btn ${vectorModes.includes(WidgetTypes.Sliders) ? 'btn-primary' : 'btn-secondary'}`}
                        onClick={() => this.onVectorModeChange(widget, WidgetTypes.Sliders)}>Sliders</button>
              </div>
            </div>
            <div className="properties-field">
              <label>Proportionally</label>
              <input className="checkbox" type="checkbox" onChange={e => this.onMetadataChange(widget, 'Proportionally', e.target.checked ? '1' : '0')} checked={checked} />
            </div>
          </div>
        }
        {this.renderDelete()}
      </>
    );
  }

  renderDialsEdit = (available: boolean) => {
    return this.renderSlidersEdit(available);
  }

  renderSlidersEdit = (available: boolean) => {
    const { preset } = this.props;
    const widget = this.state.widget as ICustomStackProperty;
    const keys = WidgetUtilities.getPropertyKeys(widget?.propertyType);
    const vectorModes = widget?.widgets || [];

    const exposed = preset?.Exposed[widget?.property] as IExposedProperty;
    let checked = false;

    if (exposed?.Metadata?.Proportionally === '1')
      checked = true;

    return (
      <>
        {available &&
          <div className="properties-list">
            {this.renderCompatibleWidgets()}
            {this.renderPropertyLabel()}
            {this.renderMinMax()}
            <div className="properties-field">
              <label>Properties</label>
              <div className="btn-group vector-btn-group">
                {keys.map(key => <button key={key}
                                         className={`btn ${vectorModes.includes(key) ? 'btn-primary' : 'btn-secondary'}`}
                                         onClick={() => this.onVectorModeChange(widget, key, keys)}>{key}</button>
                )}
              </div>
            </div>
            <div className="properties-field">
              <label>Proportionally</label>
              <input className="checkbox" type="checkbox" onChange={e => this.onMetadataChange(widget, 'Proportionally', e.target.checked ? '1' : '0')} checked={checked} />
            </div>
          </div>
        }
        {this.renderDelete()}
      </>
    );
  }

  onFunctionParameterChanged = (widget: ICustomStackWidget, param: string, value?: any) => {
    const metadata = { ...this.state.metadata };
    metadata[param] = value;
    this.setState({ metadata });
    _.set(widget, param, value);
    this.onViewUpdatedDebounced();
  };

  renderFunctionEdit = () => {
    const widget = this.state.widget as ICustomStackProperty;
    const { preset } = this.props;
    const properties = [];

    const func = preset?.Exposed[widget?.property] as IExposedFunction;
    if (func?.UnderlyingFunction?.Arguments) {
      const { metadata } = this.state;
      const ueLabel = func.Metadata?.Description || func.DisplayName;
      const label = metadata.label || widget.label || '';
      properties.push(<div key="label" className="property properties-field">
        <label>Label</label>
        <input type="text" placeholder={ueLabel} value={label} onChange={e => this.onFunctionParameterChanged(widget, 'label', e.target.value)} />
      </div>);
      for (let i = 0; i < func.UnderlyingFunction.Arguments.length; i++) {
        const arg = func.UnderlyingFunction.Arguments[i];

        const value = metadata[arg.Name] ?? widget.args?.[arg.Name];
        const argName = `args.${arg.Name}`;
        switch (arg.Type) {
          case PropertyType.String:
          case PropertyType.Text:
            properties.push(<div key={i} className="property properties-field">
              <label>{arg.Name}</label>
              <input type="text" value={value || ''} onChange={e => this.onFunctionParameterChanged(widget, argName, e.target.value)} />
            </div>);
            break;

          case PropertyType.Uint8:
          case PropertyType.Boolean:
            properties.push(<div key={i} className="property properties-field">
              <input className="checkbox" type="checkbox" checked={!!value} onChange={e => this.onFunctionParameterChanged(widget, argName, e.target.checked)} />
              <label>{arg.Name}</label>
            </div>);
            break;

          case PropertyType.Int8:
          case PropertyType.Int16:
          case PropertyType.Int32:
          case PropertyType.Uint16:
          case PropertyType.Uint32:
          case PropertyType.Float:
            properties.push(<div key={i} className="property properties-field">
              <label>{arg.Name}</label>
              <input type="number" value={value || ''} onChange={e => this.onFunctionParameterChanged(widget, argName, +e.target.value)} />
            </div>);
            break;
        }
      }
    }

    properties.push(this.renderDelete());
    return properties;
  }

  renderProperties = (available: boolean) => {
    const { selected } = this.props;
    const widget = this.state.widget as ICustomStackProperty;
    const panel = this.state.widget as IPanel;

    if (!selected || !widget)
      return null;
    
    switch (widget?.widget || panel.type) {
      case WidgetTypes.Tabs:
        return this.renderTabsEdit();

      case 'LIST':        
        return this.renderListsEdit();

      case 'PANEL':
        return this.renderPanelEdit();

      case WidgetTypes.ColorPickerList:
        return this.renderColorPickersListEdit();

      case WidgetTypes.ColorPicker:
      case WidgetTypes.MiniColorPicker:
        return this.renderColorPickerEdit(available);

      case WidgetTypes.Label:
        return this.renderLabelEdit();

      case WidgetTypes.Spacer:
        return this.renderSpacerEdit();

      case WidgetTypes.Dial:
        return this.renderDialEdit(available);

      case WidgetTypes.Dials:
        return this.renderDialsEdit(available);

      case WidgetTypes.Vector:
        return this.renderVectorEdit(available);

      case WidgetTypes.Sliders:
        return this.renderSlidersEdit(available);

      case WidgetTypes.Button:
        return this.renderFunctionEdit();
    }

    return (
      <>
        {available &&
          <div className="properties-list">
            {this.renderCompatibleWidgets()}
            {this.renderPropertyLabel()}
            {this.renderMinMax()}
          </div>
        }
        {this.renderDelete()}
      </>
    );
  }

  isPropertyUsed = (id: string) => {
    const { usedProperties } = this.state;
    return usedProperties[id];
  }

  isPropertyAvailable = () => {
    const { preset } = this.props;
    const widget = this.state.widget as ICustomStackProperty;
    return !!(widget?.property && preset?.Exposed[widget.property]);
  }

  onWidgetTypeChanged = (widget: ICustomStackProperty, type: WidgetType) => {
    this.onMetadataChange(widget, 'Widget', type);
    
    if (widget.widget) {
      widget.widget = type;
      this.props.onUpdateView();
    }
  }

  renderCompatibleProperties = (available: boolean) => {
    const { selected, preset } = this.props;
    const widget = this.state.widget as ICustomStackProperty;

    if (!selected || !widget || !widget.widget)
      return null;

    switch (widget.widget) {
      case WidgetTypes.Button:
      case WidgetTypes.ColorPicker:
      case WidgetTypes.MiniColorPicker:
      case WidgetTypes.Dial:
      case WidgetTypes.Dials:
      case WidgetTypes.Dropdown:
      case WidgetTypes.ImageSelector:
      case WidgetTypes.Joystick:
      case WidgetTypes.Slider:
      case WidgetTypes.Sliders:
      case WidgetTypes.Text:
      case WidgetTypes.Toggle:
      case WidgetTypes.Vector:
        break;

      default:
        return null;
    }

    const compatible = _.filter(preset.Exposed, w => w.Metadata.Widget === widget.widget);
    const onPropertyChange = (id: string) => {
      const element = preset.Exposed[id];

      widget.propertyType = element?.Type;
      widget.property = id;

      this.props.onUpdateView();
    };

    return (
      <div className="types-selection">
        <div className="dropdown-widget properties-field">
          <label>Property</label>
          {!available && <FontAwesomeIcon icon={['fas', 'exclamation-circle']} className="unbound-icon" />}
          <select className="dropdown" value={widget.property || ''} onChange={e => onPropertyChange(e.target.value)}>
            {!available && <option value="">Unbound Property</option>}
            {compatible.map(option => 
              <option key={option.ID} value={option.ID}>{this.isPropertyUsed(option.ID) ? '• ' : ' '}{option.DisplayName}</option>
            )}
          </select>
        </div>
      </div>
    );
  }

  renderCompatibleWidgets = () => {
    const { preset } = this.props;
    const widget = this.state.widget as ICustomStackProperty;
    if (!widget || !preset)
      return null;

    const property = preset.Exposed[widget.property];
    const compatible = WidgetUtilities.getCompatibleWidgets(property?.Type);
    const noValue = !property?.Metadata?.Widget || !compatible.find(w => property.Metadata.Widget === w);

    return (
      <div className="dropdown-widget properties-field">
        <label>Widget</label>
        {!!noValue && <FontAwesomeIcon icon={['fas', 'exclamation-circle']} className="unbound-icon" />}
        <select className="dropdown" value={property?.Metadata?.Widget || ''} onChange={e => this.onWidgetTypeChanged(widget, e.target.value)}>
          {!!noValue &&
            <option value="">[ No Widget Selected ]</option>
          }
          {compatible.map(option => <option key={option} value={option}>{option}</option>)}
        </select>
      </div>
    );
  }
  

  renderDraggableItem = (element: IExposedProperty & IExposedFunction, index: number) => {
    const { selected } = this.props;
    let draggableId = `${element.DisplayName}_`;

    switch (element.Metadata?.Widget) {
      case WidgetTypes.Button:
        draggableId += WidgetTypes.Button;
        break;

      case WidgetTypes.ColorPicker:
        draggableId += WidgetTypes.ColorPicker;
        break;

      case WidgetTypes.MiniColorPicker:
        draggableId += WidgetTypes.MiniColorPicker;
        break;

      case WidgetTypes.Toggle:
        draggableId += WidgetTypes.Toggle;
        break;

      case WidgetTypes.Dial:
        draggableId += WidgetTypes.Dial;
        break;

      case WidgetTypes.Dials:
        draggableId += WidgetTypes.Dials;
        break;

      default:
        draggableId += 'WIDGET';
        break;
    }

    let widgetClassName = 'draggable-widget control-widget ';
    let property = _.last(selected?.split('_')) || null;
    let isDragDisabled = !element.Metadata?.Widget?.length;

    if (this.isPropertyUsed(element.ID))
      widgetClassName += 'widget-in-use ';

    if (property === element.ID)
      widgetClassName += 'selected ';

    if (isDragDisabled)
      widgetClassName += 'disabled ';

    const widget = (
      <div className={widgetClassName} data-tooltip={element.DisplayName}>
        <img src="/images/GripDotsBlocks.svg" alt="Drag" className="grip-handle"/>
        <label>{element.DisplayName}</label>
        <div className="item-icon">
          {element.Metadata?.Widget}
        </div>
      </div>
    );

    return (
      <Draggable key={draggableId} draggableId={draggableId} index={index} isDragDisabled={isDragDisabled}>
        {(provided, snapshot) => (
          <>
            {snapshot.isDragging && widget}
            <li {...provided.draggableProps}
                {...provided.dragHandleProps}
                ref={provided.innerRef}
                style={this.getStyle(provided.draggableProps.style, snapshot)}
                onClick={this.props.onSelected.bind(this, `__${element.ID}`)}>
              {!snapshot.isDragging ? widget : (
                <div className="dragging-overlay">
                  {this.renderWidget(this.getWidgetFromElement(element))}
                </div>
              )}
            </li>
          </>
        )}
      </Draggable>
    );
  }

  renderTab = (tab: { title: string, key: TabType }) => {
    const { view } = this.props;
    const { title, key } = tab;

    if (!view.tabs.length && key !== TabType.Presets)
      return null;

    let className = 'drawer-nav-tab ';

    if (this.state.tab === key)
      className += 'active';

    return (
      <li className={className} key={key} onClick={() => this.setState({ tab: key })}>
        {title}
      </li>
    );
  }

  renderTabContent = () => {
    const { preset } = this.props;
    const { filteredGroups, filteredWidgets, filteredPresets, tab } = this.state;

    switch (tab) {
      case TabType.Properties:
        return <Tabs.Properties groups={filteredGroups}
                                preset={preset}                                
                                renderDraggableItem={this.renderDraggableItem}
                                onWidgetsLock={exposed => this.props.lockWidget(exposed.map(this.getWidgetFromElement))}
                                onSerach={this.onPropertiesFilter} />;

      case TabType.Tab:
        return <Tabs.Tab onChangeIcon={this.props.onChangeIcon}
                         onRenameTabModal={this.props.onRenameTabModal}
                         onDuplicateTab={this.props.onDuplicateTab} />;

      case TabType.Widgets:
        return <Tabs.Widgets widgets={filteredWidgets}
                             renderWidget={this.renderWidget}
                             onSearch={this.onWidgetsFilter} />;

      case TabType.Presets:
        return <Tabs.Pressets preset={preset}
                              pressets={filteredPresets}
                              onPresetChange={this.props.onPresetChange}
                              onSearch={this.onPresetFilter} />;
    }
  }

  render() {
    if (!this.props.editable)
      return null;

    const { editTab } = this.state;
    let className = 'drawer list ';

    if (editTab)
      className += 'tab-editable';

    const available = this.isPropertyAvailable();

    return (
      <nav id="tabs-drawer-layout" className={className}>
        <ul className="drawer-nav">
          {tabs.map(this.renderTab)}
        </ul>
        <div className="tab-content">
          {this.renderTabContent()}
        </div>
        <div className='nav-properties'>
          <div className="properties-grid">
            {this.renderCompatibleProperties(available)}
            {this.renderProperties(available)}
          </div>
        </div>
      </nav>
    );
  }
}