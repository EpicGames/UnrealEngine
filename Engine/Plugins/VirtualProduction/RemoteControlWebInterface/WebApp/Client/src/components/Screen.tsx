import React from 'react';
import { connect } from 'react-redux';
import { ReduxState } from 'src/reducers';
import { IScreen, IPayload, ScreenType } from 'src/shared';
import * as Screens from './Screens';


type PropsFromState = Partial<{
  payload: IPayload;
}>;

type Props = {
  screen: IScreen;
  visible?: boolean;
};

const mapStateToProps = (state: ReduxState): PropsFromState => ({
  payload: state.api.payload
});

@(connect(mapStateToProps) as any)
export class Screen extends React.Component<Props & PropsFromState> {

  render() {
    const { screen, payload, visible } = this.props;
    if (!screen)
      return null;

    const props = {
      payload,
      visible,
      data: screen.data,
    };

    switch (screen.type) {
      case ScreenType.Snapshot:
        return <Screens.Snapshot {...props} />;

      case ScreenType.Sequencer:
        return <Screens.Sequencer {...props} />;

      case ScreenType.Playlist:
        return <Screens.Playlist {...props} />;
      
      case ScreenType.ColorCorrection:
        return <Screens.ColorCorrection {...props} />;
      
      case ScreenType.LightCards:
        return <Screens.LightCards {...props} />;
    }

    return null;
  }
}