import React from 'react';
import { _api } from 'src/reducers';
import _ from 'lodash';
import { Button, ColorPicker, Search, Slider, TabPane, Tabs, VectorControl, SliderWheel, ValueInput } from '../../controls';
import { AlertModal } from '../../../components';
import { AssetWidget } from '../../Widgets';
import { CorrectionColorPicker, PaginationContent } from './';
import { PropertyType, PropertyValue } from 'src/shared';
import { FontAwesomeIcon } from '@fortawesome/react-fontawesome';
import { IconProp } from '@fortawesome/fontawesome-svg-core';

export enum Section {
  ColorCorrection = 'ColorCorrection',
  LightCards      = 'LightCards',
}

export enum CCRMode {
  Global      = 'Global',
  Highlights  = 'Highlights',
  Midtones    = 'Midtones',
  Shadows     = 'Shadows',
  Color       = 'Color',
}

export enum ColorProperty {
  Gain        = 'Gain',
  Gamma       = 'Gamma',
  Saturation  = 'Saturation',
  Contrast    = 'Contrast',
  Offset      = 'Offset',
}

export enum ColorMode {
  Rgb         = 'RGB',
  Hsv         = 'HSV'
}

enum ActorsGroup {
  All       = 'All',
  Favorites = 'Favorites',
}

export enum SliderMode {
  Infinity    = 'Infinity',
  Sliders     = 'Sliders'
}

export enum Sensitivity {
  x025 = '0.25',
  x05 = '0.5',
  x1 = '1',
  x2 = '2',
  x5 = '5',
}

type Action = {
  timestamp: number;
  target: string;
  property: string;
  value: any;
}

type Props = {
};

type State = {
  selected: string;
  actors: Record<string, string>;
  values: Record<string, any>;
  favorites: Record<string, boolean>;
  mode: CCRMode;
  value: any;
  root: any;
  colorProperty: ColorProperty;
  range: { min: number, max: number };
  search: string;
  section: Section;
  actorsGroup: ActorsGroup;
  sliderMode: SliderMode;
  sensitivity: Sensitivity;
};

export class ColorCorrection extends React.Component<Props, State> {

  state: State = {
    selected: '',
    actors: {},
    values: {},
    favorites: {},
    mode: CCRMode.Global,
    value: {},
    root: {},
    colorProperty: ColorProperty.Gain,
    range: { min: 0, max: 2 },
    search: '',
    section: Section.ColorCorrection,
    actorsGroup: ActorsGroup.All,
    sliderMode: SliderMode.Infinity,
    sensitivity: Sensitivity.x1,
  };

  timer: NodeJS.Timeout;
  actorsLength = 15;
  updateSelection = false;
  actionsStack: Action[] = [];
  
  async componentDidMount() {
    try {
      const favorites = localStorage.getItem('favorites');
      if (favorites)
        this.setState({ favorites: JSON.parse(favorites) });
    } catch {
    }

    await this.onRefresh();
    this.timer = setInterval(this.onRefresh, 1000);
  }

  componentDidUpdate(prevProps: Readonly<Props>, prevState: Readonly<State>) {
    const { colorProperty, value, mode } = this.state;

    if (colorProperty !== prevState.colorProperty || mode !== prevState.mode) {
      const range = this.getRange(_.get(value, this.getColorProperty()));
      this.setState({ range });
    }
  }

  componentWillUnmount() {
    clearInterval(this.timer);
    this.timer = null;
  }

  onRefresh = async () => {
    await this.onRefreshActors();
    await this.onRefreshValues();
  }

  onActorSelect = (selected: string) => {
    const { values } = this.state;
    const value = values[selected] ?? {};
    this.setState({ selected, value }, () => this.onRefreshValues(true));
  }

  onToggleActorEnable = (selected: string) => {
    const { values, section } = this.state;
    const value = values[selected] ?? {};
    switch (section) {
      case Section.ColorCorrection:
        this.onPropertyChange('Enabled', !value.Enabled);
        break;
    
      case Section.LightCards:
        this.onPropertyChange('bHidden', !value.bHidden);
        break;
    }
  }

  onSpawnActor = async () => {
    const { section } = this.state;
    this.updateSelection = true;
    let Class;

    switch (section) {
      case Section.ColorCorrection:
        Class = '/Script/ColorCorrectRegions.ColorCorrectionRegion';
        break;

      case Section.LightCards:
        Class = '/Script/DisplayCluster.DisplayClusterLightCardActor';
        break;

      default:
        return;
    }

    await _api.proxy.function(
      '/Script/RemoteControlWebInterface.Default__RCWebInterfaceBlueprintLibrary',
      'SpawnActor',
      { Class }
    );

    await this.onRefreshActors();
  }

  onDeleteActor = async() => {
    const { selected, section } = this.state;
    if (!selected)
      return;

    if (!await AlertModal.show(`Delete ${section === Section.ColorCorrection ? 'Color Correct Region' : 'Light Card'}?`))
      return;

    await _api.proxy.function(selected, 'DestroyActor');
    this.onRefreshActors();
  }

  onRefreshActors = async () => {
    const { section } = this.state;
    let Class;
    switch (section) {
      case Section.ColorCorrection:
        Class = '/Script/ColorCorrectRegions.ColorCorrectRegion';
        break;

      case Section.LightCards:
        Class = '/Script/DisplayCluster.DisplayClusterLightCardActor';
        break;

      default:
        return;
    }

    const actors = await _api.proxy.function(
      '/Script/RemoteControlWebInterface.Default__RCWebInterfaceBlueprintLibrary',
      'FindAllActorsOfClass',
      { Class }
    ) ?? {};

    const timestamp = Date.now();
    const values = await _api.proxy.function(
      '/Script/RemoteControlWebInterface.Default__RCWebInterfaceBlueprintLibrary',
      'GetValuesOfActorsByClass',
      { Class }
    ) ?? {};

    _.remove(this.actionsStack, action => action.timestamp <= timestamp);

    for (const actor in values)
      values[actor] = JSON.parse(values[actor]);

    for (const action of this.actionsStack)
      _.set(values[action.target], action.property, action.value);

    if (!_.isEqual(this.state.actors, actors) || !_.isEqual(this.state.values, values)) {
      let { selected } = this.state;

      if (selected && !actors[selected])
        selected = undefined;

      if (!selected)
        selected = _.first(Object.keys(actors));

      if (this.updateSelection)
        selected = _.last(Object.keys(actors));

      const value = values[selected];

      this.updateSelection = false;
      this.setState({ actors, values, selected, value });
    }
  }

  getRootComponentPath = () => {
    const { section, selected } = this.state;
    if (!selected)
      return selected;

    let root = '.Root';
    if (section === Section.LightCards)
      root = '.DefaultSceneRoot';
    

    let index = selected.lastIndexOf("'");
    if (index < 0)
      return `${selected}${root}`;

    return selected.slice(0, index) + root + selected.slice(index);
  }

  onRefreshValues = async (update = false) => {
    let { selected, range } = this.state;
    if (!selected)
      return;

    //const value = await _api.proxy.put('/remote/object/property', { objectPath: selected, access: 'READ_ACCESS' });
    const root = await _api.proxy.put('/remote/object/property', { objectPath: this.getRootComponentPath(), access: 'READ_ACCESS' });

    if (!_.isEqual(this.state.root, root)) {
      // if (update || _.isEmpty(this.state.value))
      //   range = this.getRange(_.get(value, this.getColorProperty()));
      this.setState({ /*value,*/ root, range });
    }
  }

  onRootPropertyValueChangeInternal = async (propertyName: string, property: string, value: any) => {
    const { selected, root } = this.state;
    if (!selected)
      return;

    const propertyValue = _.set({}, _.compact([propertyName, property]), value);
    _.merge(root, propertyValue);
    this.setState({ root });

    await _api.proxy.put('/remote/object/property', {
      objectPath: this.getRootComponentPath(),
      access: 'WRITE_TRANSACTION_ACCESS',
      propertyName,
      propertyValue
    });
  }

  onPropertyValueChangeInternal = async (propertyName: string, property: string, value: any) => {
    const { selected } = this.state;
    if (!selected)
      return;

    const propertyValue = _.set({}, property, value);
    const action: Action = { timestamp: Date.now(), target: selected, property, value };

    this.actionsStack.push(action);

    await _api.proxy.put('/remote/object/property', {
      objectPath: selected,
      access: 'WRITE_TRANSACTION_ACCESS',
      propertyName,
      propertyValue
    });
  }

  onThrottledPropertyValueChange = _.throttle(this.onPropertyValueChangeInternal, 100);
  onThrottledRootPropertyValueChange = _.throttle(this.onRootPropertyValueChangeInternal, 100);

  onPropertyChange = (property: string, v: any) => {
    const { value } = this.state;
    _.set(value, property, v);
    this.setState({ value });

    const [propertyName] = property.split('.');
    this.onThrottledPropertyValueChange(propertyName, property, v);
  }

  onReset = async (property: string, value: PropertyValue) => {
    if (!await AlertModal.show('Are you sure you want to reset?'))
      return;

    this.onPropertyChange(property, value);  
  }

  onFavoriteToggle = () => {
    const { selected, favorites, actorsGroup } = this.state;
    const actor = favorites[selected];

    if (actorsGroup === ActorsGroup.Favorites && !actor)
      return;

    favorites[selected] = !favorites[selected];
    if (!favorites[selected])
      delete favorites[selected];

    localStorage.setItem('favorites', JSON.stringify(favorites));
    
    this.setState({ favorites });
  }

  onSectionChange = (section: Section) => {
    let { mode } = this.state;

    switch (section) {
      case Section.ColorCorrection:
        mode = CCRMode.Global;
        break;

      case Section.LightCards:
        mode = CCRMode.Color;
        break;
    }

    this.setState({ section, mode, actors: {}, selected: undefined, root: undefined }, this.onRefresh);
  }

  onInfiniteWheelMove = (value: number, sign: number, min: number, max: number, property: string) => {
    const { sensitivity } = this.state;
    const step = sign * 0.0025 * (max - min) * +sensitivity;

    value = Math.max(min, Math.min(max, value + step));
    this.onPropertyChange(property, value);
  }

  getRange = (color) => {
    const { mode, colorProperty } = this.state;
    if (mode === CCRMode.Color)
      return ({ min: 0, max: 1 });

    let min = 0;
    let max = 2;

    if (colorProperty === ColorProperty.Offset) {
      min = -1;
      max = 1;
    }

    for (const key in color) {
      if (color[key] > max)
        max = color[key];
    }

    return { min, max };
  }

  getColorProperty = () => {
    const { section, mode, colorProperty } = this.state;

    if (section === Section.LightCards)
      return 'Color';

    return `ColorGradingSettings.${mode}.${colorProperty}`;
  }

  renderColorSlider = (property: string, min: number, max: number, reset: number, type: string) => {
    return this.renderSlider(property, min, max, reset, 2, null, false, type);
  }

  renderSlider = (property: string, min: number, max: number, reset?: number, precision: number = 2, label?: string, disabled?: boolean, type: string = '') => {
    const { value, sliderMode } = this.state;
    const v = _.get(value, property);

    let className = `control wrap ${property.replace(' ', '-').replace('.', '-')}-slider `;

    if (disabled)
      className += 'disabled ';

    let slider = <Slider showLabel={true}
                         precision={precision}
                         value={v}
                         onChange={value => this.onPropertyChange(property, value)}
                         min={min}
                         max={max} />;

    if (sliderMode === SliderMode.Infinity)
      slider = (
        <div className="infinite-slider-container">
          <SliderWheel size={200}
                       className={`infinite-slider ${type}`}
                       onWheelMove={sign => this.onInfiniteWheelMove(v, sign, min, max, property)} />
          <ValueInput draggable={false}
                      precision={precision}
                      min={min}
                      max={max}
                      value={v}
                      onChange={value => this.onPropertyChange(property, value)} />
        </div>
      );


    return (
      <div className={className} key={property}>
        <label>{label ?? _.startCase(_.last(property.split('.')))}</label>
        {slider}
        <FontAwesomeIcon icon={['fas', 'undo']} onClick={() => this.onPropertyChange(property, reset)} />
      </div>
    );
  }

  renderToggle = (property: string, label?: string) => {
    const { value } = this.state;

    return (
      <div className={`prop ${property.replaceAll(' ', '-').replaceAll('.', '-')}-toggle`}>
        <label>{label ?? _.startCase(property)}</label>
        <div className={`boolean-toggle ${(!_.get(value, property)) && 'checked'} `}>
          <label className="switch toggle-mode">
            <input type="checkbox" checked={!_.get(value, property)} onChange={e => this.onPropertyChange(property, !e.target.checked)} />
            <span className="slider inline"></span>
          </label>
          <div className='labels'>
            <div className='off'>ON</div>
            <div className='on'>OFF</div>
          </div>
        </div>
      </div>
    );
  }

  renderSelect = (property: string, options: string[] | Record<string, string>) => {
    const { value } = this.state;

    let opt = [];
    if (Array.isArray(options)) {
      opt = options.map(value => ({ value, label: _.startCase(value) }));
    } else {
      for (const value in options)
        opt.push({ value, label: options[value] });
    }

    return (
      <div className='prop'>
        <label>{_.startCase(property)}</label>
        <select className='dropdown' 
                value={_.get(value, property)}
                onChange={e => this.onPropertyChange(property, e.target.value)}>
          {opt.map(option => <option value={option.value} key={option.value}>{option.label}</option>)}
        </select>
      </div>
    );
  }

  executeButton = (func: string) => {
    const { selected } = this.state;
    return _api.proxy.function(selected, func, {});
  }

  renderButton = (func: string) => {
    return (
      <div className="prop">
        <Button label={func} onExecute={() => this.executeButton(func)} />
      </div>
    );
  }

  renderGuidesColorPicker = () => {
    const { value } = this.state;

    const color = _.get(value, 'GuideColor');

    return (
      <div className="guides-color-picker">
        <div className="label">Guide Color</div>
        <div className="color-controls-row">
          <ColorPicker value={color}
                       reset={true}
                       mode={ColorMode.Rgb}
                       type={PropertyType.LinearColor}
                       onChange={value => this.onPropertyChange('GuideColor', value ?? { R: 1, G: 1, B: 0, A: 0 })} />
          <div className="control-rows">
            {this.renderColorSlider('GuideColor.R', 0, 1, 1, 'red-slider')}
            {this.renderColorSlider('GuideColor.G', 0, 1, 1, 'green-slider')}
            {this.renderColorSlider('GuideColor.B', 0, 1, 1, 'blue-slider')}
          </div>
        </div>
      </div>
    );
  }

  isColorModeModified = (actor: string, mode: CCRMode) => {
    const { values, section } = this.state;

    const grading = _.get(values, [actor, (section === Section.ColorCorrection ? 'ColorGradingSettings' : 'color grading'), mode], {});
    for (const color in grading) {
      const def = (color === ColorProperty.Offset ? 0 : 1);
      for (const property in grading[color]) {
        const v = grading[color][property];
        if (v !== def)
          return true;
      }
    }

    return false;
  }

  renderChangeIndication = (actor: string, mode: CCRMode) => {
    let className = 'indicator ';
    if (this.isColorModeModified(actor, mode))
      className += 'active ';

    return (
      <div className={className}>
        <FontAwesomeIcon icon={['fas', 'circle']} />
      </div>
    );
  }

  renderTabsIcons = () => {
    const { favorites, selected } = this.state;

    let icon: IconProp = ['far', 'star'];
    if (favorites[selected])
      icon = ['fas', 'star'];

    return <>
      <span className='tab-icon' onClick={this.onFavoriteToggle}><FontAwesomeIcon icon={icon} /></span>
      {/* <span className='tab-icon'><FontAwesomeIcon icon={['fas', 'crosshairs']} /></span> */}
    </>;
  }

  getActorClassName = (actor: string) => {
    const { selected, section, values } = this.state;

    let className = 'light-card ';
    if (selected === actor)
      className += 'active ';

    let disabled;
    const value = values[actor];
    switch (section) {
      case Section.ColorCorrection:
        disabled = (value?.Enabled === false);
        break;
    
      case Section.LightCards:
        disabled = (value?.bHidden === true);
        break;
    }

    if (disabled === true)
      className += 'disabled ';

    return className;
  }

  renderActorsContent = (pagination: number, items: number, actors: Record<string, any>, favorite: boolean) => {
    const { search, favorites, section, values } = this.state;

    let start = pagination * items;
    let end = start + items;
 
    if (favorite)
      actors = Object.fromEntries(_.toPairs(actors).filter(([key, value]) => value));

    let actorsKeys = Object.keys(actors);
    if (search)
      actorsKeys = actorsKeys.filter(actor => this.state.actors[actor].toLowerCase().includes(search));

    return (
      <>
        {actorsKeys.slice(start, end).map(actor =>
          <div key={actor}
               className={this.getActorClassName(actor)}
               onClick={() => this.onActorSelect(actor)}
               onDoubleClick={() => this.onToggleActorEnable(actor)}>
            {_.startCase(this.state.actors[actor])}
            {!!favorites[actor] && 
              <FontAwesomeIcon className="icon-favorite" icon={['fas', 'star']} />
            }

            { section === Section.ColorCorrection &&
              <div className='change-indicators'>
                {this.renderChangeIndication(actor, CCRMode.Global)}
                {this.renderChangeIndication(actor, CCRMode.Highlights)}
                {this.renderChangeIndication(actor, CCRMode.Midtones)}
                {this.renderChangeIndication(actor, CCRMode.Shadows)}
              </div>
            }
          </div>
        )}
        {!favorite && (
          <div className="light-card add-button"
               onClick={this.onSpawnActor}>
            <FontAwesomeIcon icon={['fas', 'plus']} />
          </div>
        )}
      </>
    );
  }

  renderAdvancedContent = (pagination: number): Element[] => {
    const contents = [];

    if (pagination === 0)
      contents.push(
        this.renderSlider('ColorGradingSettings.ShadowsMax', 0, 1, 0),
        this.renderSlider('ColorGradingSettings.HighlightsMax', 1, 10, 0),
        this.renderSlider('ColorGradingSettings.HighlightsMin', -1, 1, 0),
      );

    if (pagination === 1)
      contents.push(
        this.renderSlider('Inner', 0, 1, 0.5),
        this.renderSlider('Outer', 0, 1, 1, 3),
        this.renderSlider('Priority', -1, 20, 0, 0),
      );

    return contents;
  }

  renderAppearanceContent = (pagination: number): Element[] => {
    const contents = [];

    if (pagination === 0)
      contents.push(
        this.renderSlider('Temperature', 0, 10_000, 6_500),
        this.renderSlider('Tint', -1, 1, 0, 3),
        this.renderSlider('Exposure', -2, 10, 0.5, 3, 'Exposure'),
        this.renderSlider('Gain', -2, 10, 1, 3),
      );

    if (pagination === 1)
      contents.push(
        this.renderSlider('Opacity', 0, 1, 1, 3),
        this.renderSlider('Feathering', 0, 1, 0, 3),
        <div key="none1" />,
        <div key="none2" />,
      );

    return contents;
  }

  renderOrientationContent = (pagination: number) => {
    const contents = [];

    if (pagination === 0)
      contents.push(
        this.renderSlider('DistanceFromCenter', -2_000, 2_000, 300),
        this.renderSlider('Longitude', 0, 360, 0, 2),
        this.renderSlider('Latitude', -90, 90, 0),
        this.renderSlider('Spin', -360, 360, 0),
        this.renderSlider('Pitch', -360, 360, 0),
        this.renderSlider('Yaw', -360, 360, 0),
      );

    if (pagination === 1)
      contents.push(
        this.renderSlider('Scale.X', 0, 20, 2, 3, 'Scale X'),
        this.renderSlider('Scale.Y', 0, 20, 2, 3, 'Scale Y'),
        this.renderSlider('RadialOffset', -360, 360, 0),
        <div key="empty" />,
      );

    return contents;
  }

  renderPositionTab = () => {
    const { root, sensitivity } = this.state;

    return (
      <>
        <div className="control-rows no-space vector-controls">
          <div className="control">
            <VectorControl value={_.get(root, 'RelativeLocation')}
                           label="Location"
                           propertyType={PropertyType.Vector}
                           step={+sensitivity}
                           defaultValue={{ X: 0, Y: 0, Z: 0}}
                           onChange={(property, value) => this.onThrottledRootPropertyValueChange('RelativeLocation', property, value)} />
          </div>
          <div className="control">
            <VectorControl value={_.get(root, 'RelativeRotation')}
                           label="Rotation"
                           propertyType={PropertyType.Rotator}
                           step={+sensitivity}
                           defaultValue={{ Pitch: 0, Roll: 0, Yaw: 0}}
                           onChange={(property, value) => this.onThrottledRootPropertyValueChange('RelativeRotation', property, value)} />
          </div>
          <div className="control">
            <VectorControl value={_.get(root, 'RelativeScale3D')}
                           label="Scale"
                           propertyType={PropertyType.Vector}
                           step={+sensitivity}
                           defaultValue={{ X: 1, Y: 1, Z: 1}}
                           onChange={(property, value) => this.onThrottledRootPropertyValueChange('RelativeScale3D', property, value)} />
          </div>
        </div>
      </>
    );
  }

  renderSection = () => {
    const { section, value } = this.state;

    switch (section) {
      case Section.ColorCorrection:
        return (
          <Tabs rightIcon={this.renderTabsIcons()}
                defaultActiveKey="default">
            <TabPane id="position" tab="Position">
              {this.renderPositionTab()}
            </TabPane>
            <TabPane id="default" tab="Default">
              <div className='props-row fit-dropdown'>
                {this.renderSelect('Type', ['Sphere', 'Box', 'Cylinder', 'Cone'])}
                {this.renderSelect('TemperatureType', ['LegacyTemperature', 'WhiteBalance', 'ColorTemperature'])}
                {this.renderToggle('Enabled')}
              </div>
              <div className='control-rows'>
                {this.renderSlider('Temperature', 0, 10_000, 6_500)}
                {this.renderSlider('Intensity', 0, 1, 1)}
                {this.renderSlider('Falloff', 0, 20, 0, 3)}
                {this.renderSlider('Tint', -1, 1, 0, 3)}
              </div>
            </TabPane>
            <TabPane id="advanced" tab="Advanced">
              <div className='props-row'>
                {this.renderToggle('Invert')}
              </div>
              <div className='multilines-content'>
                <PaginationContent pages={2} key="advanced">
                  {({ pagination }) => this.renderAdvancedContent(pagination)}
                </PaginationContent>
              </div>
            </TabPane>
          </Tabs>
        );

      case Section.LightCards:
        return (
          <Tabs rightIcon={this.renderTabsIcons()}
                defaultActiveKey="appearance">
            <TabPane id="appearance" tab="Appearance">
              <div className="grid-row">
                <div className="props-row">
                  {this.renderSelect('Mask', ['Square', 'Circle', 'Polygon', 'UseTextureAlpha'])}
                  <div className="prop asset-control">
                    <AssetWidget browse
                                 reset={false}
                                 type="Texture"
                                 typePath="/Script/Engine.Texture"
                                 label="Texture"
                                 onChange={v => this.onPropertyChange('Texture', v)}
                                 value={_.get(value, 'Texture') ?? ''} />
                    <span className="reset"><FontAwesomeIcon icon={['fas', 'undo']} onClick={() => this.onPropertyChange('Texture', null)} /></span>
                  </div>
                </div>
              </div>
              <div className="multilines-content">
                <PaginationContent pages={2} key="appearance">
                  {({ pagination }) => this.renderAppearanceContent(pagination)}
                </PaginationContent>
              </div>
            </TabPane>
            <TabPane id="orientation" tab="Orientation">
              <div className="multilines-content full-height">
                <PaginationContent pages={2} key="orientation">
                  {({ pagination }) => this.renderOrientationContent(pagination)}
                </PaginationContent>
              </div>
            </TabPane>
            {/* <TabPane id="guides" tab="Guides">
              <div className='props-row'>
                {this.renderToggle('bShowGuides', 'Show Guides')}
              </div>
              <div className='control-rows'>
                {this.renderSlider('GuideScale', 0, 1, 0)}
                {this.renderGuidesColorPicker()}
              </div>
            </TabPane> */}
            <TabPane id="position" tab="Position">
              {this.renderPositionTab()}
            </TabPane>
          </Tabs>
        );
    }

    return null;
  }

  render() {
    const { selected, mode, actors, favorites, actorsGroup, section, value, colorProperty, range, sliderMode, sensitivity } = this.state;
    const style: React.CSSProperties = {};
    const property = this.getColorProperty();
    const color = _.get(value, property);

    style.position = 'relative';

    let className = 'cc-group cc-panel ';

    if (!selected)
      className += 'disabled';

    let availableFavorites = _.pickBy(favorites, (v, k) => !!actors[k]);

    return (
      <div className={`color-correction-wrapper custom-ui-wrapper ${section}`} tabIndex={-1}>
        <div className='ipad-debug'></div>
        <div className="group">
          <CorrectionColorPicker section={section}
                                 property={property}
                                 color={color}
                                 colorProperty={colorProperty}
                                 range={range}
                                 mode={mode}
                                 disabled={!selected}
                                 sliderMode={sliderMode}
                                 sensitivity={sensitivity}
                                 onSectionChange={this.onSectionChange}
                                 onPropertyChange={this.onPropertyChange}
                                 onRangeUpdate={color => this.setState({ range: this.getRange(color) })}
                                 onColorPropertyChange={colorProperty => this.setState({ colorProperty })}
                                 onColorModeChange={mode => this.setState({ mode })}
                                 isColorPropertyModified={mode => this.isColorModeModified(selected, mode)} />
        </div>
        <div className="group">
          <div className="cc-group shotbox cc-panel">
            <Tabs rightIcon={<>
              <div className="trash-icon" onClick={this.onDeleteActor}><FontAwesomeIcon icon={['fas', 'trash']} /></div>
              <Search placeholder="Search" onSearch={search => this.setState({ search })} />
            </>}
                  onTabChange={(actorsGroup: ActorsGroup) => this.setState({ actorsGroup })}
                  defaultActiveKey={actorsGroup}>
              <TabPane id={ActorsGroup.All} tab="All" icon={['fas', 'square']}>
                <PaginationContent pageItems={this.actorsLength}
                                   total={Object.values(actors).length}
                                   key="all">
                  {({ pagination }) => this.renderActorsContent(pagination, this.actorsLength, actors, false)}
                </PaginationContent>
              </TabPane>
              <TabPane id={ActorsGroup.Favorites} tab="Favorites" icon={['fas', 'star']}>
                <PaginationContent pageItems={this.actorsLength + 1}
                                   total={Object.values(availableFavorites).length}
                                   key="favorites">
                  {({ pagination }) => this.renderActorsContent(pagination, this.actorsLength, availableFavorites, true)}
                </PaginationContent>
              </TabPane>
            </Tabs>
          </div>
          <div className={className} style={style}>
            <div className="body">
              {this.renderSection()}
            </div>
          </div>
        </div>
        <div className="group bottom-bar">
          <div className="cc-group inline-group">
            <div className="group-item operation-style bottom-tabs-style">
              <div className="label">
                Operation Style
              </div>
              <Tabs onlyHeader
                    defaultActiveKey={sliderMode}
                    onTabChange={(sliderMode: SliderMode) => this.setState({ sliderMode })}>
                {Object.values(SliderMode).map(mode => <TabPane key={mode}
                                                                tab={mode}
                                                                id={mode} />)}
              </Tabs>
            </div>
            <div className="group-item sensitivity-style bottom-tabs-style">
              <div className="label">
                Sensitivity
              </div>
              <Tabs onlyHeader
                    defaultActiveKey={sensitivity}
                    onTabChange={(sensitivity: Sensitivity) => this.setState({ sensitivity })}>
                {Object.values(Sensitivity).map(mode => <TabPane key={mode}
                                                                 tab={`x${mode}`}
                                                                 id={mode} />)}
              </Tabs>
            </div>
          </div>
        </div>
      </div>
    );
  }
}
