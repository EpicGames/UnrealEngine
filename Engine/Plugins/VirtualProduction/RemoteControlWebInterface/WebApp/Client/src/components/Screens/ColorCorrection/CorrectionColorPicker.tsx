import React from 'react';
import { ReactComponent as CCIcon } from '../../../assets/cc_icon.svg';
import { ReactComponent as LCIcon } from '../../../assets/lc_icon.svg';
import { ColorPicker, SliderWheel, TabPane, Tabs, ValueInput } from '../../controls';
import { FontAwesomeIcon } from '@fortawesome/react-fontawesome';
import { Section, ColorProperty, CorrectionColorSlider } from './';
import { AlertModal } from 'src/components/modals';
import { WidgetUtilities } from 'src/utilities';
import { CCRMode, ColorMode, Sensitivity, SliderMode } from './ColorCorrection';
import { PropertyType, VectorProperty } from 'src/shared';
import _ from 'lodash';


type Props = {
  section: Section;
  property: string;
  color: any;
  colorProperty: ColorProperty;
  range: { min: number, max: number };
  mode: CCRMode;
  sliderMode: SliderMode;
  sensitivity: Sensitivity;
  disabled: boolean;

  onSectionChange: (section: Section) => void;
  onColorPropertyChange: (colorProperty: ColorProperty) => void;
  onPropertyChange: (property: string, v: any) => void;
  onRangeUpdate: (color: any) => void;
  onColorModeChange: (mode: CCRMode) => void;
  isColorPropertyModified: (mode: CCRMode) => boolean;
}

type State = {
  colorMode: ColorMode,
}

export class CorrectionColorPicker extends React.Component<Props, State> {

  state: State = {
    colorMode: ColorMode.Rgb,
  }

  onReset = async (property: string, value) => {
    if (!await AlertModal.show('Are you sure you want to reset?'))
      return;

    this.props.onPropertyChange(property, value);
  }

  onColorModeChange = () => {
    let colorMode = ColorMode.Hsv;

    if (this.state.colorMode === ColorMode.Hsv)
      colorMode = ColorMode.Rgb;
    
    this.setState({ colorMode });
  }

  onColorChange = (v: any) => {
    const { color, property } = this.props;

    if (_.isObject(color))
      v = { ...color, ...v };

    return this.props.onPropertyChange(property, v);
  }

  onHsvWheelMove = (sign: number, hue?: number) => {
    let { range, color, mode, sensitivity } = this.props;
    let rgb = WidgetUtilities.colorToRgb(color, range.max);

    const hsv = WidgetUtilities.rgb2Hsv(rgb);

    if (hue)
      hsv.h = hue / 360;
    else {
      let step = sign * 0.001 * +sensitivity;
      if (hsv.h + step < 0)
        step += 1;

      hsv.h = hsv.h + step;
    }

    rgb = WidgetUtilities.hsv2rgb(hsv);
    color = WidgetUtilities.rgbToColor(rgb, range.max, mode !== CCRMode.Color);

    this.onColorChange(color);
  }
  
  onShiftSliderMove = (sign: number) => {
    const { colorProperty, property, color, sensitivity } = this.props;
    const step = sign * 0.01 * +sensitivity;

    if (!color)
      return;

    let sum = color.X + color.Y + color.Z;
    let min = 0;

    if (colorProperty === ColorProperty.Offset)
      min = -1;

    color.X = Math.max(min, color.X + ((color.X / sum) || 1) * step);
    color.Y = Math.max(min, color.Y + ((color.Y / sum) || 1) * step);
    color.Z = Math.max(min, color.Z + ((color.Z / sum) || 1) * step);

    this.props.onPropertyChange(property, color);
    this.props.onRangeUpdate(color);
  }

  getSectionProps = (value: Section) => {
    const { section, onSectionChange } = this.props;
    let className = 'cc-group mode-button ';

    if (section === value)
      className += 'active ';

    return { className, onClick: () => onSectionChange(value) };
  }

  getColorResetValue = () => {
    const { mode, colorProperty } = this.props;
    if (mode === CCRMode.Color)
      return ({ R: 1, G: 1, B: 1, A: 0});

    if (colorProperty === ColorProperty.Offset)
      return ({ X: 0, Y: 0, Z: 0, W: 0 });

    return ({ X: 1, Y: 1, Z: 1, W: 1 });
  }

  getHsvValue = () => {
    const { range, color } = this.props;

    const rgb = WidgetUtilities.colorToRgb(color, range.max);
    const hsv = WidgetUtilities.rgb2Hsv(rgb);

    return hsv;
  }

  getValueColor = (): React.CSSProperties => {
    const { color, mode } = this.props;

    if (!color || mode === CCRMode.Color)
      return { opacity: 0 };

    const max = Math.max(color.X, color.Y, color.Z, 1);

    const rgb = WidgetUtilities.colorToRgb(color, max);
    const hsv = WidgetUtilities.rgb2Hsv(rgb);

    let opacity = 0;
    if (hsv.v)
      opacity = Math.max(0, Math.min(1, +(1 - hsv.v).toFixed(4)));

    return { opacity };
  }

  getPreviewColorStyles = (color: VectorProperty): React.CSSProperties => {
    const { range } = this.props;
    const rgb = WidgetUtilities.colorToRgb(color, range.max);

    return { background: `rgb(${Math.round(rgb.R)},${Math.round(rgb.G)},${Math.round(rgb.B)})` };
  }

  getPropertyType = () => {
    const { mode } = this.props;
    let propertyType = PropertyType.Vector4;

    if (mode === CCRMode.Color)
      propertyType = PropertyType.LinearColor;

    return propertyType;
  }

  renderColorSlider = (sliderProperty: string, label: string, className: string) => {
    const { colorMode } = this.state;
    const { color, colorProperty, property, range, sliderMode, sensitivity } = this.props;
    const propertyType = this.getPropertyType();

    return <CorrectionColorSlider className={className}
                                  label={label}
                                  mode={sliderMode}
                                  sensitivity={+sensitivity}
                                  color={color}
                                  colorProperty={colorProperty}
                                  property={property}
                                  sliderProperty={sliderProperty}
                                  range={range}
                                  colorMode={colorMode}
                                  propertyType={propertyType}
                                  onPropertyChange={this.props.onPropertyChange}
                                  onRangeUpdate={this.props.onRangeUpdate} />;
  }

  renderSliders = () => {
    const propertyType = this.getPropertyType();

    if (propertyType === PropertyType.LinearColor) {
      return (
        <>
          {this.renderColorSlider('R', 'R', 'red-slider')}
          {this.renderColorSlider('G', 'G', 'green-slider')}
          {this.renderColorSlider('B', 'B', 'blue-slider')}
          {this.renderColorSlider('A', 'A', 'alpha-slider')}      
        </>
      );
    }
    
    if (this.state.colorMode === ColorMode.Hsv) {
      return (
        <>
          {this.renderColorSlider('s', 'S', 'saturation-slider')}
          {this.renderColorSlider('v', 'V', 'alpha-slider')}
          {this.renderColorSlider('a', 'Y', 'alpha-slider')}
        </>
      );
    }

    return (
      <>
        {this.renderColorSlider('X', 'R', 'red-slider')}
        {this.renderColorSlider('Y', 'G', 'green-slider')}
        {this.renderColorSlider('Z', 'B', 'blue-slider')}
        {this.renderColorSlider('W', 'Y', 'alpha-slider')}      
      </>
    );
  }

  renderModifyIndicator = (mode: CCRMode) => {
    if (!this.props.isColorPropertyModified?.(mode))
      return null;

    return <div className="change-indicator active"><FontAwesomeIcon icon={['fas', 'circle']} /></div>;
  }

  render() {
    const { property, color, colorProperty, range, mode, disabled, section } = this.props;
    const { colorMode } = this.state;

    const reset = this.getColorResetValue();
    const hsv = this.getHsvValue();

    const hue = hsv ? (hsv.h * 360).toFixed(1) : 0;
    const style: React.CSSProperties = {};

    if (disabled) {
      style.opacity = 0.6;
      style.pointerEvents = 'none';
    }

    let modes = Object.values(CCRMode);
    if (section === Section.ColorCorrection)
      modes = modes.filter(m => m !== CCRMode.Color);

    return (
      <div className="group">
        <div className="cc-group screen-mode-selectors">
          <div className="mode-selector">
            <div {...this.getSectionProps(Section.ColorCorrection)}>
              <CCIcon />
            </div>
            <div {...this.getSectionProps(Section.LightCards)}>
              <LCIcon />
            </div>
          </div>
        </div>
        <div className="cc-group" style={style}>
          <div className="body" style={{ width: "calc(100% - 20px)" }}>
            {section === Section.ColorCorrection &&
              <div className="mode-selectors">
                <Tabs onTabChange={this.props.onColorModeChange}
                      onlyHeader
                      defaultActiveKey={mode}>
                  {modes.map(mode => <TabPane key={mode}
                                              id={mode} 
                                              tab={mode}
                                              view={() => this.renderModifyIndicator(mode)} />
                  )}
                </Tabs>
              </div>
            }
            <div className="color-picker-singleton">
              <div className="color-wheel">
                <FontAwesomeIcon className="reset-icon" icon={['fas', 'undo']} onClick={() => this.onReset(property, reset)} />
                {colorMode === ColorMode.Rgb && <div className="color-picker-value" style={this.getValueColor()} />}
                <ColorPicker value={color}
                             mode={colorMode}
                             alpha={mode === CCRMode.Color}
                             type={this.getPropertyType()}
                             max={range.max}
                             onChange={value => this.onColorChange(value ?? reset)} />
                <div className="wheel-addon">
                  <div className="dot" style={this.getPreviewColorStyles(color)}></div>
                  <div className={`color-mode-toggle ${(colorMode === ColorMode.Hsv) && 'checked'} `}>
                    <label className="switch toggle-mode">
                      <input type="checkbox" checked={colorMode === ColorMode.Hsv} onChange={this.onColorModeChange} />
                      <span className="slider inline"></span>
                    </label>
                    <div className="labels">
                      <div className="off">RGB</div>
                      <div className="on">HSV</div>
                    </div>
                  </div>
                </div>
              </div>
              <div className="shift-slider">
                {colorMode === ColorMode.Rgb && <>
                  <SliderWheel vertical
                               onWheelMove={this.onShiftSliderMove} />
                  Value
                </>
                }
              </div>
              <div className="details-section">
                <div className="color-property-selection">
                  {section === Section.ColorCorrection && Object.values(ColorProperty).map(property =>
                    <div key={property}
                         onClick={() => this.props.onColorPropertyChange(property)}
                         className={`mode-selector ${property === colorProperty ? 'active' : ''}`}>{property}</div>
                  )}
                </div>
                <div className="color-properties">
                  <div className="sliders">
                    <div className="rgb-sliders">
                      {this.renderSliders()}
                      <div className="slider-wheel-container hue-slider">
                        <span className="title">Hue</span>
                        <SliderWheel size={200} onWheelMove={sign => this.onHsvWheelMove(sign)} />
                        <ValueInput draggable={false}
                                    precision={1}
                                    min={0}
                                    max={360}
                                    value={hue}
                                    onChange={hue => this.onHsvWheelMove(1, hue)} />
                      </div>
                    </div>
                  </div>
                </div>
              </div>
            </div>
          </div>
        </div>
      </div>
    );
  }
};