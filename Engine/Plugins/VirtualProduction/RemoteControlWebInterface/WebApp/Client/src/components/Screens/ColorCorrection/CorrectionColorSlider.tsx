import React from 'react';
import { SliderMode, ColorMode } from './ColorCorrection';
import { SliderWheel, Slider, ValueInput } from '../../controls';
import { ColorProperty } from './ColorCorrection';
import { WidgetUtilities } from 'src/utilities';
import { PropertyType } from 'src/shared';


type Props = {
  className?: string;
  label: string;
  mode: SliderMode;
  sensitivity: number;
  color: any;
  colorProperty: ColorProperty;
  property: string;
  sliderProperty: string;
  colorMode: ColorMode;
  range: { min: number, max: number };
  propertyType: PropertyType;

  onPropertyChange: (property: string, v: any) => void;
  onRangeUpdate: (color: any) => void;
}

type State = {

}

export class CorrectionColorSlider extends React.Component<Props, State> {

  onChange = (key: string, sign: number, v?: number) => {
    const { colorMode } = this.props;

    if (colorMode === ColorMode.Rgb)
      return this.onRgbChange(key, sign, v);

    return this.onHsvChange(key, sign, v);
  }

  onRgbChange = (key: string, sign: number, v?: number) => {
    const { sensitivity, color, colorProperty, property } = this.props;

    if (!color)
      return;

    const step = sign * 0.01 * sensitivity;
    color[key] += step;

    if (v !== undefined)
      color[key] = v;

    if (colorProperty !== ColorProperty.Offset)
      color[key] = Math.max(0, color[key]);

    this.props.onPropertyChange(property, color);
    this.props.onRangeUpdate(color);
  }

  onHsvChange = (key: string, sign: number, v?: number) => {
    const { sensitivity, propertyType, range, colorProperty, property } = this.props;

    if (!this.props.color)
      return;

    const step = sign * 0.01 * sensitivity;
    const hsv = this.getHsv(range.max);

    hsv[key] += step;

    if (v !== undefined) {
      if(key === 'v')
        v = v / range.max;

      hsv[key] = v;
    }

    hsv[key] = Math.max(hsv[key], 0.0001);
    
    const rgb = WidgetUtilities.hsv2rgb(hsv);
    let color = rgb as any;

    switch (propertyType) {
      case PropertyType.Vector4:
        color = WidgetUtilities.rgbToColor(rgb, range.max);
        break;

      case PropertyType.LinearColor:
        color = WidgetUtilities.rgbToColor(rgb, range.max, false);
        break;
    }

    if (rgb.A)
      color.W = rgb.A;

    if (colorProperty !== ColorProperty.Offset)
      for (const key in color)
        color[key] = Math.max(0, color[key]);

    this.props.onPropertyChange(property, color);
    this.props.onRangeUpdate(color);
  }

  getHsv = (max = 1) => {
    const { color } = this.props;
    const rgb = WidgetUtilities.colorToRgb(color, max);
    
    return WidgetUtilities.rgb2Hsv(rgb);
  }

  getValue = (key: string) => {
    const { colorMode } = this.props;

    if (colorMode === ColorMode.Rgb)
      return this.getRgbValue(key);

    return this.getHsvValue(key);
  }

  getRgbValue = (key: string) => {
    const { color } = this.props;
    const value = color?.[key] as number;

    return value?.toFixed(3) ?? 0;
  }

  getHsvValue = (key: string) => {
    const hsv = this.getHsv();
    return hsv[key];
  }
  
  getSSliderColor = () => {
    const hsv = this.getHsv();
    const rgb = WidgetUtilities.hsv2rgb({ h: hsv.h, s: 1, v: 1, a: hsv?.a });

    return `linear-gradient(90deg, ${this.getVSliderMinColor()} 0%, rgba(${rgb.R}, ${rgb.G}, ${rgb.B}, ${rgb?.A ?? 1}) 100%)`;
  }

  getVSliderColor = () => {
    const hsv = this.getHsv();
    const rgb = WidgetUtilities.hsv2rgb({ h: hsv.h, s: hsv.s, v: 1, a: hsv?.a });

    return `linear-gradient(90deg, rgba(0,0,0,${rgb?.A ?? 1}) 0%, rgba(${rgb.R}, ${rgb.G}, ${rgb.B}, ${rgb?.A ?? 1}) 100%)`;
  }

  getVSliderMinColor = () => {
    const hsv = this.getHsv();
    const rgb = WidgetUtilities.hsv2rgb({ h: hsv.h, s: 0, v: hsv.v, a: hsv?.a });

    return `rgba(${rgb.R}, ${rgb.G}, ${rgb.B}, ${rgb?.A ?? 1})`;
  }

  render() {
    let { className = '', mode, label, sliderProperty, range, colorMode } = this.props;
    let sliderMax = Math.max(range.min, range.max);

    const wheel = mode === SliderMode.Infinity;
    const style: React.CSSProperties = {};

    className += ` slider-wheel-container ${sliderProperty}-slider`;

    if (!wheel)
      className += ` slider`;

    if (colorMode === ColorMode.Hsv && sliderProperty === 's') {
      style.background = this.getSSliderColor();
      sliderMax = 1;
    };

    if (colorMode === ColorMode.Hsv && sliderProperty === 'v') {
      style.background = this.getVSliderColor();
    };
      
    return (
      <div className={className}>
        <span className="title">{label}</span>
        {wheel &&
          <SliderWheel size={200}
                       style={style}
                       onWheelMove={sign => this.onChange(sliderProperty, sign)} />
        }
        {!wheel &&
          <Slider showLabel={false}
                  style={style}
                  value={+this.getValue(sliderProperty)}
                  onChange={value => this.onChange(sliderProperty, 0, value)}
                  min={range.min}
                  max={sliderMax}
                  size={200} />
        }
        <ValueInput draggable={false}
                    precision={3}
                    min={range.min}
                    value={this.getValue(sliderProperty)}
                    onChange={value => this.onChange(sliderProperty, 0, value)} />
      </div>
    );
  }
};