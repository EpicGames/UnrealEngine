import React from 'react';


type Props = {
  pageItems?: number;
  total?: number;
  pages?: number;

  children: ({ ...state }: State) => any;
}

type State = {
  pagination: number;
}

export class PaginationContent extends React.Component<Props, State> {

  state: State = {
    pagination: 0,
  }

  getIsDisabled = (dir: number) => {
    const { pagination } = this.state;
    const { pageItems, total, pages } = this.props;

    let step = pagination + dir;

    if (step < 0)
      return true;

    if (pages && step > pages - 1)
      return true;

    return (step * pageItems) >= total;
  }

  onPaginationClick = (dir: number, disabled: boolean) => {
    if (disabled)
      return;

    let { pagination } = this.state;
    pagination += dir;

    this.setState({ pagination });
  }

  renderPaginationArrow = (dir: number) => {
    let className = 'arrow up ';
    const disabled = this.getIsDisabled(dir);

    if (dir > 0)
      className = className.replace('up', 'down');

    if (disabled)
      className += 'disabled ';

    return (
      <div className={className} 
           onClick={() => this.onPaginationClick(dir, disabled)}>
        <div className='icon' />
      </div>
    );
  }

  render() {
    const { children } = this.props;

    return (
      <div className="content" style={{ display: 'flex' }}>
        <div className="body" style={{ width: '100%' }}>
          {children({ ...this.state })}
        </div>
        <div className="arrows-pagination">
          {this.renderPaginationArrow(-1)}
          {this.renderPaginationArrow(1)}
        </div>
      </div>
    );
  }
};