import React from 'react';
import { _api } from 'src/reducers';
import _ from 'lodash';
import { ColorPicker, Slider } from '../controls';
import { PropertyType } from 'src/shared';
import { FontAwesomeIcon } from '@fortawesome/react-fontawesome';


type Props = {
};

type State = {
  selected: string;
  actors: Record<string, string>;
  value: any;
};


export class LightCards extends React.Component<Props, State> {

  timer: NodeJS.Timeout;
  state: State = {
    selected: '',
    actors: {},
    value: {},
  };

  async componentDidMount() {
    await this.onRefresh();
    this.timer = setInterval(this.onRefresh, 1000);
  }

  componentWillUnmount() {
    clearInterval(this.timer);
    this.timer = null;
  }

  onRefresh = async () => {
    await this.onRefreshActors();
    await this.onRefreshValues();
  }

  onActorSelect = (selected: string) => {
    this.setState({ selected }, this.onRefreshValues);
  }

  onSpawnActor = async () => {
    await _api.proxy.function(
      '/Script/RemoteControlWebInterface.Default__RCWebInterfaceBlueprintLibrary',
      'SpawnActor',
      {
        Class: '/nDisplay/LightCard/LightCard.LightCard_C'
      });

    await this.onRefreshActors();
  }

  onRefreshActors = async () => {
    const actors: Record<string, string> = await _api.proxy.function(
      '/Script/RemoteControlWebInterface.Default__RCWebInterfaceBlueprintLibrary',
      'FindAllActorsOfClass',
      {
        Class: '/nDisplay/LightCard/LightCard.LightCard_C'
      });

    if (!_.isEqual(this.state.actors, actors)) {
      let { selected } = this.state;
      
      if (selected && !actors[selected])
        selected = undefined;

      if (!selected)
        selected = _.first(Object.keys(actors));
        
      this.setState({ actors, selected });
    }
 
  }

  onRefreshValues = async () => {
    const { selected } = this.state;
    if (!selected)
      return;

    const value = await _api.proxy.put('/remote/object/property', { objectPath: selected, access: 'READ_ACCESS' });
    if (!_.isEqual(this.state.value, value)) {
      this.setState({ value });
    }
  }

  onPropertyValueChangeInternal = async (propertyName: string, property: string, value: any) => {
    const { selected } = this.state;
    if (!selected)
      return;
    
    const propertyValue = _.set({}, property, value);
    await _api.proxy.put('/remote/object/property', { 
      objectPath: selected,
      access: 'WRITE_ACCESS',
      propertyName,
      propertyValue
    });
  }

  onThrottledPropertyValueChange = _.throttle(this.onPropertyValueChangeInternal, 100);

  onPropertyChange = (property: string, v: any) => {
    const { value } = this.state;
    _.set(value, property, v);
    this.setState({ value });

    const [ propertyName ] = property.split('.');
    this.onThrottledPropertyValueChange(propertyName, property, v);
  }

  renderColorPicker = (property: string) => {
    const { value } = this.state;

    return (
      <ColorPicker value={_.get(value, property)}
                    type={PropertyType.LinearColor}
                    alpha={true}
                    onChange={value => this.onPropertyChange(property, value)} />
    );
  }

  renderSlider = (property: string, min: number, max: number, label?: string) => {
    const { value } = this.state;

    return (
      <div className='control'>
        <label>{label ?? _.startCase(property)}</label>
        <Slider showLabel={true}
                value={_.get(value, property)}
                onChange={value => this.onPropertyChange(property, value)}
                min={min}
                max={max} />
    </div>
    );
  }

  render() {
    const { selected, actors } = this.state;

    const style: React.CSSProperties = {};
    if (!selected) {
      style.opacity = 0.6;
      style.pointerEvents = 'none';
    }

    return (
      <div className="light-cards-wrapper  custom-ui-wrapper" tabIndex={-1}>
        <div className='group' style={style}>

          <div className='group vertical'>
            <div className='cc-group'>
              <div className='cc-title'>Size and Position</div>
              <div className='body'>
                {this.renderSlider('DistanceFromCenter', 0, 10_000, 'Distance')}
                {this.renderSlider('Scale.X', 0.01, 1)}
                {this.renderSlider('Scale.Y', 0.01, 1)}
                {this.renderSlider('Longitude', 0, 360)}
                {this.renderSlider('Latitude', 0, 360)}
              </div>
            </div>

            <div className='cc-group'>
              <div className='cc-title'>Guides</div>
              <div className='body'>
                {this.renderColorPicker('Color')}
              </div>
            </div>
          </div>


          <div className='cc-group'>
            <div className='cc-title'>Appearance</div>
            <div className='body'>
              {this.renderColorPicker('GuideColor')}

              {this.renderSlider('Exposure', 0, 10)}
              {this.renderSlider('Opacity', 0, 1)}
              {this.renderSlider('Feathering', 0, 10)}
            </div>
          </div>

        </div>

        <div className='cc-group shotbox'>
          <div className='body'>
            {Object.keys(actors).map(actor => 
              <div key={actor}
                  className={`light-card ${selected === actor ? 'active' : ''}`}
                  onClick={() => this.onActorSelect(actor)}>
                {_.startCase(actors[actor])}
              </div>
            )}

            <div className="light-card active"
                onClick={() => this.onSpawnActor()}>
              <FontAwesomeIcon icon={['fas', 'plus']} />
            </div>                
          </div>
        </div>
      </div>
    );
  }
}
