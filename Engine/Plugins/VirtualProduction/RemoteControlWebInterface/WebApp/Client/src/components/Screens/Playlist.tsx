import React from 'react';
import { FontAwesomeIcon } from '@fortawesome/react-fontawesome';
import { _api } from 'src/reducers';
import _ from  'lodash';
import { IAsset } from 'src/shared';
import { SafeImage, ValueInput } from '../controls';
import { SearchModal } from '../';
import path from 'path';


interface IPlaylistData {
  Description: string;
  Items: string[];
  PlaylistItems: IPlaylistItem[];
}

interface IPlaylistItem {
  Path: string;
  IsPlaying?: boolean;

  Sequence: string;
	StartFrameOffset: number;
	EndFrameOffset: number;
	bHoldAtFirstFrame: boolean;
	NumLoops: number;
  PlaybackSpeed: number;
	bMute: boolean;
}

type Props = {
  visible?: boolean;
};

type State = {
  filter: string;
  sequences: IAsset[];
  playlist?: IAsset;
  data?: IPlaylistData;
  activeTooltip: number;
};

export class Playlist extends React.Component<Props, State> {
  player: string;
  interval: NodeJS.Timer;

  state: State = {
    filter: '',
    sequences: [],
    activeTooltip: null,
  };

  componentDidMount() {
    document.onclick = this.onDocumentClick;
    this.onInitialize();
    this.interval = setInterval(this.onCheckSequenceStatus, 300);
  }

  componentWillUnmount() {
    document.onclick = null;
    clearInterval(this.interval);
    this.interval = null;
  }

  onInitialize = async () => {
    const WorldContextObject = await _api.proxy.function("Object'/Engine/Transient.UnrealEdEngine_0:UnrealEditorSubsystem_0'", 'GetEditorWorld');
    if (!WorldContextObject)
      return;

    this.player = await _api.proxy.function("Object'/Script/SequencerPlaylists.Default__SequencerPlaylistPlayer'", 'GetDefaultPlayer', { WorldContextObject });

    const sequences = await _api.assets.search('', ['/Script/LevelSequence.LevelSequence'], '/Game', 200);
    this.setState({ sequences });

    const currentPlaylist = await _api.proxy.function(this.player, 'GetPlaylist');
    if (currentPlaylist) {
      const folder = path.dirname(currentPlaylist);
      const [filename] = path.basename(currentPlaylist).split('.');

      const playlistAsset = await _api.assets.search(filename, ['/Script/SequencerPlaylists.SequencerPlaylist'], folder, 200);
      await this.onLoadPlaylist(playlistAsset[0]);
    }
  }

  onSearchChange = (filter: string)  => {
    this.setState({ filter });
  }

  onCheckSequenceStatus = async () => {
    const { playlist, data } = this.state;
    if (!playlist || !data?.Items)
      return;

    let changed = false;
    for (const item of data.PlaylistItems) {
      const playing = !!await _api.proxy.function(this.player, 'IsPlaying', { Item: item.Path });

      if (playing !== item.IsPlaying) {
        item.IsPlaying = playing;
        changed = true;
      }
    }

    if (changed)
      this.forceUpdate();
  }

  onDocumentClick = (e: MouseEvent) => {
    const { activeTooltip } = this.state;

    if (e.defaultPrevented || activeTooltip === null)
      return;

    this.setState({ activeTooltip: null });
  }

  onTooltipClick = (e: React.MouseEvent, activeTooltip: number) => {
    e.preventDefault();
    this.setState({ activeTooltip });
  }

  onPlaylistBrowse = async () => {
    const playlist = await SearchModal.open({ placeholder: 'Search Playlist', types: ['/Script/SequencerPlaylists.SequencerPlaylist'], prefix: '/Game', count: 200 });
    if (!playlist)
      return;

    await _api.proxy.function(this.player, 'SetPlaylist', { InPlaylist: playlist.Path });
    await this.onLoadPlaylist(playlist);
  }

  onLoadPlaylist = async (playlist: IAsset) => {
    if (!playlist)
      return;

    const data = await _api.proxy.property.get<IPlaylistData>(playlist.Path);
    data.PlaylistItems = [];
    for (const itemPath of data.Items) {
      const item = await _api.proxy.property.get<IPlaylistItem>(itemPath);
      item.Path = itemPath;
      data.PlaylistItems.push(item);
    }

    this.setState({ playlist, data, filter: '' });
  }

  onAddItem = async () => {
    // const { playlist, data } = this.state;
    // if (!playlist || !data?.Items)
    //   return;

    // data.Items.push('');
    // await _api.proxy.property.set(playlist.Path, 'Items', data.Items);
  }

  onSequenceBrowse = async (item: IPlaylistItem) => {
    const sequence = await SearchModal.open({ placeholder: 'Search Sequence', types: ['LevelSequence'], prefix: '/Game', count: 200 });    
    if (sequence)
      await this.onSetPlaylistItem(item, 'Sequence', sequence.Path);
  }

  onPlaylistAction = (action: string) => {
    const { playlist } = this.state;
    if (!playlist || !this.player)
      return;

    return _api.proxy.function(this.player, action);
  }

  onPlaylistItemAction = (item: IPlaylistItem, action: string) => {
    const { playlist } = this.state;
    if (!playlist || !this.player)
      return;

    return _api.proxy.function(this.player, action, { Item: item.Path });
  }


  onSetPlaylistItem = async (item: IPlaylistItem, property: string, value: any) => {
    item[property] = value;
    await _api.proxy.property.set(item.Path, property, value);
    this.forceUpdate();
  }

  renderPlaylistItem = (item: IPlaylistItem, index: number) => {
    const { sequences } = this.state;

    const url = _api.assets.thumbnailUrl(item.Sequence);

    let playButtonClass: string = "";
    if (item.IsPlaying)
      playButtonClass = "active";

    return (
      <tr key={index} className={`playlist-item col col-${index + 1}`}>
        <td className="col-actions">
          <div className="controls">
            <button tabIndex={-1} className={`btn btn-secondary playlist-play ${(item.IsPlaying) && 'playing'} `} onClick={() => this.onPlaylistItemAction(item, 'PlayItem')}>
              <FontAwesomeIcon icon={['fas', 'play']} />
            </button>
            <button tabIndex={-1} className={`btn btn-secondary playlist-stop  ${(item.IsPlaying) && 'playing active'}`} onClick={() => this.onPlaylistItemAction(item, 'StopItem')}>
              <FontAwesomeIcon icon={['fas', 'stop']} />
            </button>
            <button tabIndex={-1} className="btn btn-secondary playlist-rehold" onClick={() => this.onPlaylistItemAction(item, 'ResetItem')}>
              <FontAwesomeIcon icon={['fas', 'reply']} />
            </button>
          </div>
        </td>

        <td className='col-thumb'>
          <div className="thumbnail-block">
            <SafeImage src={url} 
                        fallback="/images/assets/Sequencer.png"
                        alt={item.Sequence}
                        className="box-image" />
          </div>
        </td>
        <td className='col-name'>
          <select value={item.Sequence} onChange={e => this.onSetPlaylistItem(item, 'Sequence', e.target.value)}>
            <option>None</option>
            {sequences.map(seq => <option key={seq.Path} value={seq.Path}>{seq.Name}</option>)}
          </select>
          <span className="icon" onClick={() => this.onSequenceBrowse(item)}>
          <img src='/images/icons/playlistIcons/ContentBrowser.svg' className='playlist-icon'/>
          </span>
        </td>
        <td className='col-control'>
          <div className='input-container'>
            <ValueInput value={item.StartFrameOffset} onChange={value => this.onSetPlaylistItem(item, 'StartFrameOffset', value)} precision={0} />
          </div>
          <div className='input-container'>
            <ValueInput value={item.EndFrameOffset} onChange={value => this.onSetPlaylistItem(item, 'EndFrameOffset', value)} precision={0} />
          </div>
        </td>
        <td className='col-control'>
          
          <div className='checkbox-container'>
            <label className="container">
              <input type="checkbox" checked={item.bHoldAtFirstFrame} onChange={e => this.onSetPlaylistItem(item, 'bHoldAtFirstFrame', e.target.checked)}/>
              <span className="checkmark">
                <img src='/images/icons/playlistIcons/HoldFrame.svg' className='playlist-icon'/>
              </span>
            </label>
          </div>
        </td>
        <td className='col-control'>
          <div>
            <ValueInput value={item.NumLoops} onChange={value => this.onSetPlaylistItem(item, 'NumLoops', value)} precision={0} />
          </div>
        </td>
      </tr>
    );
  }

  render() {
    const { filter, playlist, data } = this.state;

    let items = data?.PlaylistItems ?? [];
    if (filter) {
      const filterLowerCase = filter.toLowerCase();
      items = _.filter(items, i => i.Sequence.toLowerCase().includes(filterLowerCase));
    }

    if (!playlist)
      return (
        <div className="playlist-wrapper">
          <div className="playlist-placeholder">
            <div className="playlist-placeholder-title" onClick={this.onPlaylistBrowse}>
              <img src='/images/icons/playlistIcons/PlaylistLoad.svg' className='playlist-icon large'/>
              Load Playlist
            </div>
          </div>
        </div>
      );

    return (
      <div className="playlist-wrapper" tabIndex={-1}>
        <div className="playlist-controls">
          <div>
            <div className="playlist-header" onClick={this.onPlaylistBrowse}>
              <span className="icon"><FontAwesomeIcon icon={['fas', 'folder']} /></span>
              <h1>{playlist?.Name}</h1>
            </div>
            <span className="description">{data?.Description}</span>
          </div>
          <div className="controls">
            <button tabIndex={-1} className="btn btn-secondary playlist-play" onClick={() => this.onPlaylistAction('PlayAll')}>
              <FontAwesomeIcon icon={['fas', 'play']} />
            </button>
            <button tabIndex={-1} className="btn btn-secondary playlist-stop" onClick={() => this.onPlaylistAction('StopAll')}>
              <FontAwesomeIcon icon={['fas', 'stop']} />
            </button>
            <button tabIndex={-1} className="btn btn-secondary playlist-rehold" onClick={() => this.onPlaylistAction('ResetAll')}>
              <FontAwesomeIcon icon={['fas', 'reply']} />
            </button>
          </div>
        </div>
        <div className="list-wrapper">
          <div className="search-block">
          <button tabIndex={-1} className="btn btn-secondary add-item" onClick={this.onAddItem}><FontAwesomeIcon icon={['fas', 'plus']} /> Item</button>
            <div className="search-field">
              <FontAwesomeIcon icon={['fas', 'search']} />
              <input type="text" value={filter} onChange={e => this.onSearchChange(e.target.value)} autoComplete="no" />
            </div>
          </div>

          <div className="table-wrapper">
            <table>
              <thead>
                <tr>
                  <th className="col col-control">&nbsp;</th>
                  <th className="col col-3" colSpan={2}>Playlist Items</th>
                  <th className="col col-control">Offset</th>
                  <th className="col col-control">Hold</th>
                  <th className="col col-control">Loop</th>
                </tr>
              </thead>
              <tbody>
                {items.map(this.renderPlaylistItem)}
              </tbody>
            </table>
          </div>
        </div>
      </div>
    );
  }
}

type PlaylistTooltipProps = {
  visible: boolean;
}


class PlaylistTooltip extends React.Component<PlaylistTooltipProps> {
  render() {
    const { visible } = this.props;

    if (!visible)
      return null;

    return (
      <div className="playlist-tooltip">
        <div className="playlist-tooltip-header">Item Details</div>
      </div>
    );
  }
}
