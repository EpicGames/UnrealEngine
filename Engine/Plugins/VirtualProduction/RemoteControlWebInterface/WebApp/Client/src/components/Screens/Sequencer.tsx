import React from 'react';
import { FontAwesomeIcon } from '@fortawesome/react-fontawesome';
import { _api } from 'src/reducers';
import _ from  'lodash';
import { IAsset } from 'src/shared';
import { SafeImage } from '../controls';


type Props = {
  visible?: boolean;
};

type State = {
  search: string;
  assets: IAsset[],
};

export class Sequencer extends React.Component<Props, State> {
  state: State = {
    search: '',
    assets: [],
  };

  componentDidMount() {
    this.fetchAssets();
  }

  componentDidUpdate(prevProps: Props) {
    if (this.props.visible && !prevProps.visible)
      this.fetchAssets();
  }

  fetchAssets = async () => {
    const { search } = this.state;
    const assets = await _api.assets.search(search, ['/Script/LevelSequence.LevelSequence'], '/Game', 200);
    this.setState({ assets });
  }

  fetchAssetsDebounced = _.debounce(this.fetchAssets, 200);

  onSearchChange = (search: string)  => {
    this.setState({ search });
    this.fetchAssetsDebounced();
  }

  renderSequence = (asset: IAsset, index: number) => {
    const url = _api.assets.thumbnailUrl(asset.Path);

    return (
      <tr key={index} className={`col col-${index + 1}`}>
        <td>
          <div className="name-block">
            <SafeImage src={url} 
                        fallback="/images/assets/Sequencer.png"
                        alt={asset.Name}
                        className="box" />
            <p>{asset.Name}</p>
          </div>
        </td>
        <td><p>{asset.Path}</p></td>
        <td>
          <div className="play-icon" onClick={() => this.playSequence(asset)}>
            <FontAwesomeIcon icon={['fas', 'play']} />
          </div>
        </td>
      </tr>
    );
  }

  playSequence = async (sequence: IAsset) => {
    const sequencer = '/Script/LevelSequenceEditor.Default__LevelSequenceEditorBlueprintLibrary';
    const editor = '/Script/EditorScriptingUtilities.Default__EditorAssetLibrary';
    try {
      await _api.proxy.put('/remote/object/call', { objectPath: editor, functionName: 'LoadAsset', parameters: { 'AssetPath': sequence.Path }  });
      await _api.proxy.put('/remote/object/call', { objectPath: sequencer, functionName: 'OpenLevelSequence', parameters: { 'LevelSequence': sequence.Path }  });
      await _api.proxy.put('/remote/object/call', { objectPath: sequencer, functionName: 'Pause' });
      await _api.proxy.put('/remote/object/call', { objectPath: sequencer, functionName: 'SetCurrentTime', parameters: { NewFrame: 0 } });
      await _api.proxy.put('/remote/object/call', { objectPath: sequencer, functionName: 'Play' });
    } catch (err) {
      console.log('Failed to play sequence', sequence);
    }
  }

  render() {
    const { search, assets } = this.state;

    return (
      <div className="snapshot-wrapper sequencer-wrapper" tabIndex={-1}>
        <div className="list-wrapper">
          <div className="search-block">
            <div className="search-field">
              <FontAwesomeIcon icon={['fas', 'search']} />
              <input type="text" value={search} onChange={e => this.onSearchChange(e.target.value)} autoComplete="no" />
            </div>
          </div>

          <div className="table-wrapper">
            <table>
              <thead>
                <tr>
                  <th className="col col-2">Name</th>
                  <th className="col col-3">Folder</th>
                  <th className="col col-4"></th>
                </tr>
              </thead>
              <tbody>
                {assets.map(this.renderSequence)}
              </tbody>
            </table>
          </div>
        </div>
      </div>
    );
  }
}
