import React from 'react';
import ReactDOM from 'react-dom';
import { IAsset } from 'src/shared';
import { FontAwesomeIcon } from '@fortawesome/react-fontawesome';
import { _api } from 'src/reducers';
import _ from  'lodash';
import { SafeImage } from '../controls';

type Props = {
  visible?: boolean;
};

type State = {
  map: string;
  search: string;
  sortBy: string;
  sortAsc: boolean;
  snapshot?: IAsset;
  snapshots: IAsset[];
  filters: IAsset[];
};

export class Snapshot extends React.Component<Props, State> {

  state: State = {
    map: '',
    search: '',
    sortBy: 'Metadata.CaptureTime',
    sortAsc: false,
    snapshots: [],
    filters: [],
  };

  componentDidMount() {
    this.fetchAssets();
  }

  componentDidUpdate(prevProps: Props) {
    if (this.props.visible && !prevProps.visible)
      this.fetchAssets();
  }

  fetchAssets = async () => {
    let { map } = this.state;

    if (!map) {
	    const editorSubsystem = await _api.proxy.function('/Script/UnrealEd.Default__EditorSubsystemBlueprintLibrary', 'GetEditorSubsystem', { Class: '/Script/UnrealEd.UnrealEditorSubsystem' });
      map = await _api.proxy.function(editorSubsystem, 'GetEditorWorld', {});
    }

    const nativeFilters = await _api.assets.search('', ['/Script/LevelSnapshotFilters.LevelSnapshotFilter'], '/Game', {}, 50);

    const filterArgs = {
      EnableBlueprintNativeClassFiltering: true,
      NativeParentClasses: ['/Script/LevelSnapshotFilters.LevelSnapshotBlueprintFilter'],
    };
    const bpFilters = _.filter(await _api.assets.search('', ['/Script/Engine.Blueprint'], '/Game', filterArgs, 50), f => !!f.Metadata['GeneratedClass']);

    const filters = [...bpFilters, ...nativeFilters];

    const { search, sortBy, sortAsc } = this.state;
    const assets = await _api.assets.search(search, ['/Script/LevelSnapshots.LevelSnapshot'], '/Game', {}, 200);
    let snapshots = _.filter(assets, asset => !map || asset.Metadata?.MapPath === map);
    
    snapshots = this.sortAssets(snapshots, sortBy, sortAsc);
    this.setState({ snapshots, filters, map });
  }

  fetchAssetsDebounced = _.debounce(this.fetchAssets, 200);

  sortAssets = (snapshots: IAsset[], sortBy: string, ascending: boolean) => {
    return _.orderBy(snapshots, [sortBy], [ascending ? 'asc' : 'desc'] );
  }

  onSortChange = () => {
    const { sortBy, sortAsc } = this.state;
    const snapshots = this.sortAssets(this.state.snapshots, sortBy, sortAsc);

    this.setState({ snapshots });
  }

  onSetSort = (sortBy: string) => {
    let { sortAsc } = this.state;
    sortAsc = (sortBy === this.state.sortBy) ? !sortAsc : true;
    this.setState({ sortBy, sortAsc }, this.onSortChange);
  }

  onSearchChange = (search: string)  => {
    this.setState({ search });
    this.fetchAssetsDebounced();
  }

  showDetails = (snapshot?: IAsset) => {
    this.setState({ snapshot });
  }

  onCreate = async () => {
    const { map } = this.state;
    if (await SaveModal.show(map))
      this.fetchAssets();
  }

  onKeyPress = (e: React.KeyboardEvent<HTMLDivElement>) => {
    if (e.key === 'Escape' && this.state.snapshot)
      this.setState({ snapshot: null });
  }

  renderSnapshot = (snapshot: IAsset, index: number) => {
    let className = '';
    if (snapshot === this.state.snapshot)
      className = 'selected-row ';

    
    const url = _api.assets.thumbnailUrl(snapshot.Path);

    return (
      <tr key={index} className={className} onClick={() => this.showDetails(snapshot)}>
        <td className="name">
          <div className="name-block">
            <SafeImage src={url} 
                        fallback="/images/assets/LevelSnapshot.png"
                        alt={snapshot.Name}
                        className="box" />
            <p>{snapshot.Name}</p>
          </div>
        </td>

        <td className="description"><p>{snapshot.Metadata?.SnapshotDescription}</p></td>

        <td className="created"><p>{snapshot.Metadata?.CaptureTime}</p></td>
      </tr>
    );
  }

  renderColumnHeader = (title: string, field: string) => {
    const { sortBy, sortAsc } = this.state;

    return (
      <>
        <th className={title.toLowerCase()} onClick={() => this.onSetSort(field)}>
          {title}
          {sortBy === field &&
            <FontAwesomeIcon icon={sortAsc ? 'sort-up' : 'sort-down'} className="sort-icon" />
          }
        </th>
      </>
    );
  }

  render() {
    const { map, search, filters, snapshot, snapshots } = this.state;

    return (
      <div className="snapshot-wrapper" onClick={() => this.showDetails()} tabIndex={-1}>
        <div className="list-wrapper">
          <div className="search-block">
            <button className="btn snapshot-btn" onClick={this.onCreate}><p>TAKE SNAPSHOT</p></button>
            <div className="search-field">
              <FontAwesomeIcon icon={['fas', 'search']} />
              <input type="text" value={search} onChange={e => this.onSearchChange(e.target.value)} autoComplete="no" />
            </div>
          </div>

          <div className="table-wrapper" onClick={e => e.stopPropagation()}>
            <table>
              <thead>
                <tr>
                  {this.renderColumnHeader('Name', 'Name')}
                  {this.renderColumnHeader('Description', 'Metadata.SnapshotDescription')}
                  {this.renderColumnHeader('Created', 'Metadata.CaptureTime')}
                </tr>
              </thead>
              <tbody>
                {snapshots.map(this.renderSnapshot)}
              </tbody>
            </table>
          </div>
        </div>

        <SnapshotPreview map={map} filters={filters} snapshot={snapshot} onClose={() => this.showDetails()} />
      </div>
    );
  }
}



type SnapshotPreviewProps = {
  map: string;
  snapshot?: IAsset;
  filters: IAsset[];
  onClose: () => void;
};

type SnapshotPreviewState = {
  filter?: string;
  loading?: boolean;
};

class SnapshotPreview extends React.Component<SnapshotPreviewProps, SnapshotPreviewState> {
  state: SnapshotPreviewState = {
  };

  onLoad = async () => {
    const { map, snapshot, onClose, filters } = this.props;
    const { filter, loading } = this.state;
    if (!snapshot || loading)
      return;

    this.setState({ loading: true });

    try {

      let filterInstance;
      if (filter) {
        const filterAsset = _.find(filters, f => f.Path === filter);

        switch (filterAsset?.Class) {
          case 'Blueprint':
            filterInstance  = await _api.proxy.function(
              '/Script/LevelSnapshotFilters.Default__FilterBlueprintFunctionLibrary', 
              'CreateFilterByClass',
              { 
                Class: filterAsset.Metadata['GeneratedClass']
              }
            );
            break;

          case 'LevelSnapshotFilter':
            filterInstance = filterAsset.Path;
            break;
        }
      }

      await _api.proxy.function(
        '/Script/LevelSnapshots.Default__LevelSnapshotsFunctionLibrary',
        'ApplySnapshotToWorld',
        {
          WorldContextObject: map,
          Snapshot: snapshot.Path,
          OptionalFilter: filterInstance,
        }
      );

    } catch {
    }

    this.setState({ loading: false });
    onClose?.();
  }

  render() {
    const { snapshot, filters } = this.props;
    const { filter, loading } = this.state;
    let className = 'btn snapshot-btn ';
    
    if (!snapshot)
      className += 'disabled';

    return (
      <div className="snapshot-preview-wrapper" onClick={e => e.stopPropagation()}>
        <div className="preview-title">
          <span>{snapshot?.Metadata?.SnapshotName ? snapshot?.Name : <span className="hint-text">Select a snapshot above</span>}</span>
        </div>

        {snapshot?.Path &&
          <div className="preview-description">
            <div className="description-title">Asset Path</div>
            {snapshot?.Path}
          </div>
        }
        <div className="controls-block">
          <div className="snapshot-dropdown">
            <div className="title">Filter</div>
            <select className="dropdown" value={filter} onChange={e => this.setState({ filter: e.target.value })}>
              <option value="">[None]</option>
              {filters?.map((asset, index) =>
                <option key={index} value={asset.Path}>{_.head(asset.Name?.split('.'))}</option>
              )}
            </select>
          </div>
          <button className={className} onClick={this.onLoad}>
            {loading && <FontAwesomeIcon icon={['fas', 'sync-alt']} className="spin-icon" spin />}
            <p>APPLY SNAPSHOT</p>
          </button>
        </div>
      </div>
    );
  }
}



type SaveModalProps = {
  map: string;
  close: (result: boolean) => void;
};

type SaveModalState = {
  name: string;
  path: string;
  description: string;
  loading?: boolean;
};

class SaveModal extends React.Component<SaveModalProps, SaveModalState> {

  state: SaveModalState = {
    loading: true,
    name: '',
    path: '/Game/LevelSnapshots/',
    description: '',
  };

  private static pending: (result?: boolean) => void;

  public static show(map: string): Promise<boolean> {
    SaveModal.pending?.(undefined);
    SaveModal.pending = undefined;

    const div = document.createElement('div');
    document.getElementById('modal').prepend(div);

    return new Promise<boolean>(resolve => {
      SaveModal.pending = resolve;

      ReactDOM.render(<SaveModal map={map} close={resolve} />, div);
    })
      .finally(() => {
        const unmountResult = ReactDOM.unmountComponentAtNode(div);
        if (unmountResult && div.parentNode)
          div.parentNode.removeChild(div);
      });
  }

  async componentDidMount() {

    try {
      const editor = '/Script/LevelSnapshotsEditor.Default__LevelSnapshotsEditorSettings';

      const { RootLevelSnapshotSaveDir } = await _api.proxy.property.get(editor, 'RootLevelSnapshotSaveDir');
      let path: string = RootLevelSnapshotSaveDir?.Path ?? '/Game/LevelSnapshots';

      const { LevelSnapshotSaveDir } = await _api.proxy.property.get(editor, 'LevelSnapshotSaveDir');
      if (LevelSnapshotSaveDir)
        path += `/${LevelSnapshotSaveDir}`;

      const { DefaultLevelSnapshotName } = await _api.proxy.property.get(editor, 'DefaultLevelSnapshotName');

      let InWorldName = this.props.map;
      const match = /([^/]+)\..+$/.exec(this.props.map);
      if (match?.[1])
        InWorldName = match[1];      

      
      const PathFormatted = await _api.proxy.function(editor, 'ParseLevelSnapshotsTokensInText', { InTextToParse: path, InWorldName });
      const NameFormatted = await _api.proxy.function(editor, 'ParseLevelSnapshotsTokensInText', { InTextToParse: DefaultLevelSnapshotName, InWorldName });

      this.setState({ path: PathFormatted, name: NameFormatted });
    } catch {

    }

    this.setState({ loading: false });
  }

  onOK = async () => {
    const { name, path, description, loading } = this.state;
    if (!name || loading)
      return;

    this.setState({ loading: true });

    try {
      await _api.proxy.function(
        '/Script/LevelSnapshotsEditor.Default__LevelSnapshotsEditorFunctionLibrary',
        'TakeAndSaveLevelSnapshotEditorWorld',
        {
          FileName: name,
          FolderPath: path,
          Description: description
        }
      );
    } catch {
    }

    this.setState({ loading: false });
    this.props.close(true);
  }

  onCancel = () => {
    if (!this.state.loading)
      this.props.close(false);
  }

  onKeyPress = (e: React.KeyboardEvent<HTMLDivElement>) => {
    if (e.key === 'Escape')
      this.onCancel();
  }
  
  render() {
    const { name, path, description, loading } = this.state;

    return (
      <div className="fullscreen" onClick={this.onCancel} onKeyPress={this.onKeyPress} tabIndex={-1}>
        <div className="save-asset-modal" onClick={e => e.stopPropagation()}>
          <div className="save-row">
            TAKE SNAPSHOT
            <FontAwesomeIcon icon={['fas', 'times']} onClick={this.onCancel} />
          </div>

          <div className="item-row">
            <div className="item-title">Name</div>
            <input className="item-content name-field" 
                    value={name}
                    onChange={e => this.setState({ name: e.target.value })}
                    autoFocus
                    autoComplete="no"
                    readOnly={loading} />
          </div>

          <div className="item-row">
            <div className="item-title">Description</div>
            <textarea className="item-content description-field"
                      readOnly={loading}
                      rows={5}
                      value={description}
                      onChange={e => this.setState({ description: e.target.value })} />
          </div>

          <div className="item-row">
            <div className="item-title">Save Dir</div>
            <input className="item-content name" value={path} onChange={e => this.setState({ path: e.target.value })} autoComplete="no" readOnly={loading} />
          </div>

          <button className="btn snapshot-btn" disabled={!name} onClick={this.onOK}>
            {loading && 
              <FontAwesomeIcon icon={['fas', 'sync-alt']} className="spin-icon" spin />
            }
            TAKE SNAPSHOT
          </button>
        </div>
      </div>
    );
  }
}