import React from 'react';
import { ConnectionSignal } from 'src/shared';


type Props = {
  signal: ConnectionSignal;
}

export class SignalIcon extends React.Component<Props> {
  render() {
    const { signal } = this.props;
    if (signal === ConnectionSignal.Good)
      return null;

    return (
      <div className={`signal-icon ${signal.toLowerCase()}`}>
        <span className="bar-1"></span>
        <span className="bar-2"></span>
        <span className="bar-3"></span>
        <span className="bar-4"></span>
      </div>
    );
  }
};