import React from 'react';
import { connect } from 'react-redux';
import { _api, ReduxState } from 'src/reducers';
import { ColorProperty, ICustomStackProperty, ICustomStackTabs, ICustomStackWidget, IPanel, IPanelType, IPreset,
        IPayload, PropertyType, PropertyValue, VectorProperty, WidgetTypes, IExposedProperty, IExposedFunction, IColorPickerList, ICustomStackListItem } from 'src/shared';
import { ColorPicker, ColorPickerList, Button, Text, VectorDrawer, Dial, JoysticksWrapper, DialMode, DialsWrapper } from './controls';
import { PrecisionModal } from '../components';
import { FontAwesomeIcon } from '@fortawesome/react-fontawesome';
import { DraggableWidget, DroppableWidget, VirtualList } from '.';
import * as Widgets from './Widgets';
import { WidgetUtilities } from '../utilities';
import _ from 'lodash';
import { AlertModal } from './modals';


type PropsFromState = Partial<{
  payload: IPayload;
  preset: IPreset;
}>;

type Props = {
  editable: boolean;
  visible?: boolean;
  panels: IPanel[];
  tabKey: string;
  dragging: string;
  droppable: string;
  selected: string;
  hoverTab: string;
  vector: ICustomStackProperty;

  onSelected: (selected: string) => void;
  onSetVectorDrawer: (vector?: ICustomStackProperty) => void;
  getDroppableId: (id: string, path: string, accept: string[], type: string) => string;
};

interface IPanelTemplate {
  matching: Record<string, string>;
  selected?: string;
}

type State = {
  selection: Record<string, string>;
  draggingId: string;
  templates: Record<string, IPanelTemplate>;

  precisionModal?: { widget: ICustomStackProperty, property?: string };
};

const mapStateToProps = (state: ReduxState, props: Props): PropsFromState => ({
  payload: state.api.payload,
  preset: state.api.presets[state.api.preset],
});

const tempId = 'TEMP_ID';

@(connect(mapStateToProps) as any)
export class Stack extends React.Component<Props & PropsFromState, State> {

  state: State = {
    selection: {},
    draggingId: null,
    templates: {},
  };

  dropIndex = -1;
  root = null;  
  scrollY = null;
  isIos = false;

  componentDidMount() {
    this.root = document.querySelector('.droppable-root') as any;
    this.isIos = /iPad|iPhone|iPod/.test(navigator.platform);

    this.root.ontouchmove = (e) => {
      const touches = e.touches;
      const touch = touches[0];

      if (touches.length !== 2)
        return;

      if (!this.scrollY)
        this.scrollY = touch.pageY + this.root.scrollTop;
        
      this.root.scrollTo(0, this.scrollY - touch.pageY);
    };

    this.root.ongesturechange = (e) => e.preventDefault();    
    this.root.ongestureend = (e) => {
      e.preventDefault();
      this.scrollY = null;
    };    

    try {
      const templates = JSON.parse(sessionStorage.getItem('templates'));
      if (templates)
        this.setState({ templates });
    } catch {
    }
  }

  componentWillUnmount() {
    this.root.ontouchmove = null;
    this.root.ongesturechange = null;
    this.root.ongestureend = null;
  }

  shouldComponentUpdate(prevProps: Props) {
    const { visible } = this.props;

    return visible || prevProps.visible !== visible;
  }

  componentDidUpdate(prevProps: Props) {
    const { dragging, hoverTab } = this.props;

    if (!dragging && dragging !== prevProps.dragging)
      this.setState({ draggingId: null });

    if (hoverTab !== prevProps.hoverTab)
      this.onPointerEnter(hoverTab);
  }

  getLabel = (widget: ICustomStackProperty) => {
    if (!widget)
      return null;

    const { preset } = this.props; 
    const exposed = preset?.Exposed[widget.property];
    if (exposed)
      return widget.label || exposed.Metadata?.Description || exposed.DisplayName;

    return (
      <div className="unbound-property">
        <FontAwesomeIcon icon={['fas', 'exclamation-circle']} />
        <span>Unbound Property</span>
      </div>
    );
  }

  getPropertyValue = (property: string) => {
    if (!property)
      return;

    return this.props.payload[property];
  }

  getValue = (widget: ICustomStackProperty): any=> {
    return this.getPropertyValue(widget?.property);
  }

  onPropertyValueChange = async (widget: ICustomStackProperty, value?: PropertyValue) => {
    const { preset } = this.props;
    if (!widget?.property || !preset?.Exposed[widget.property])
      return;

    if (widget.propertyType === PropertyType.Function)
      return _api.payload.execute(widget.property, widget.args);

    if (widget.propertyType === PropertyType.StatelessController)
      return _api.payload.set(widget.property, ""); // stateless controller have no value.

    if (value === undefined && !await AlertModal.show('Reset to default?'))
      return;

    _api.payload.set(widget.property, value);
  }

  onThrottledPropertyValueChange = _.throttle((widget, value) => this.onPropertyValueChange(widget, value), 100);

  onDragChange = (draggingId: string) => {
    const { dragging } = this.props;

    if (!!dragging)
      return;

    this.setState({ draggingId });
  };

  onSelect = (prefix: string, id: string) => {
    const selection = { ...this.state.selection };

    selection[prefix] = id;
    this.setState({ selection });
  }

  onPointerEnter = (value: string) => {
    const [prefix, id] = value.split('_');
    const { selection, draggingId } = this.state;
    const { dragging } = this.props;

    if (!dragging || _.first(draggingId?.split('_')) === 'NAVIGATION')
      return;

    if (selection[prefix] !== id) {
      selection[prefix] = id;

      this.setState({ selection });
    }
  }

  getMetadata = (property: IExposedProperty | IExposedFunction, field: string) => {
    const str = property?.Metadata?.[field];
    if (str === undefined)
      return;

    const number = parseFloat(str);
    if (isNaN(number))
      return;
    
    return number;
  }

  getWidgetMetadata = (widget: ICustomStackProperty) => {
    const { preset } = this.props;
    const property = preset.Exposed[widget.property];
    const type = property?.Type;
    const precision = WidgetUtilities.getPropertyPrecision(type);
    const alpha = !!this.getMetadata(property, 'Alpha');
    const proportionally = !!this.getMetadata(property, 'Proportionally');

    let min = this.getMetadata(property, 'Min');
    let max = this.getMetadata(property, 'Max');

    if (type === PropertyType.Rotator) {
      if (min === undefined)
        min = -180;
      if (max === undefined)
        max = 180;
    }

    return { type, min, max, precision, alpha, proportionally };
  }

  getAlpha = (property: string) => {
    const { preset } = this.props;
    const meta = preset.Exposed[property]?.Metadata;
    return meta?.Alpha === '1';
  }

  renderPrecisionModal = () => {
    const { precisionModal } = this.state;
    if (!precisionModal?.widget || !precisionModal.widget?.property)
      return null;

    const { preset } = this.props;
    const { widget, property } = precisionModal;
    if (!preset?.Exposed[widget.property])
      return null;

    const meta = this.getWidgetMetadata(widget);
    const label = this.getLabel(widget);
    const type = widget.propertyType;

    let value = this.getValue(widget);
    let onChange;
    
    switch (type) {
      case PropertyType.Rotator:
      case PropertyType.Vector:
        value = value[property];
        onChange = (newValue) => this.onPropertyAxisValueChange(widget, property, newValue);
        break;

      default:
        onChange = (newValue) => this.onThrottledPropertyValueChange(widget, newValue);
        break;
    }

    return (
      <PrecisionModal meta={meta}
                      widget={widget.widget}
                      property={widget.property}
                      label={label}
                      type={type}
                      value={value}
                      onChange={onChange}
                      onMetadataChange={(key, value) => this.onMetadataChange(widget.property, key, value)}
                      onClose={() => this.setState({ precisionModal: null })} />
    );
  }

  getWidgetId = (widget: ICustomStackWidget, index: number) => {
    if (!widget.id)
      return tempId + index;

    return `${widget.id}_WIDGET`;
  }

  renderSlider = (widget: ICustomStackProperty, index: number, path: string) => {
    const id = this.getWidgetId(widget, index);
    const meta = this.getWidgetMetadata(widget);
    const selection = `${path}_${index}_${widget.property}`;

    return (
      <DraggableWidget key={id}
                       draggableId={id}
                       index={index}
                       className="slider"
                       onPointerChange={this.onDragChange}
                       onSelect={this.props.onSelected.bind(this, selection)}
                       selected={this.props.selected === selection}
                       isDragDisabled={!this.props.editable}>
        <Widgets.SliderWidget value={this.getValue(widget)}
                              min={meta.min}
                              max={meta.max}
                              precision={meta.precision}
                              label={this.getLabel(widget)}
                              onPrecisionModal={() => this.setState({ precisionModal: { widget } })}
                              onChange={value => this.onThrottledPropertyValueChange(widget, value)} />
      </DraggableWidget>
    );
  }

  renderSliders = (widget: ICustomStackProperty, index: number, path: string) => {
    const id = this.getWidgetId(widget, index);
    const meta = this.getWidgetMetadata(widget);
    const selection = `${path}_${index}_${widget.property}`;

    return (
      <DraggableWidget key={id}
                       draggableId={id}
                       className="sliders"
                       index={index}
                       onPointerChange={this.onDragChange}
                       onSelect={this.props.onSelected.bind(this, selection)}
                       selected={this.props.selected === selection}
                       isDragDisabled={!this.props.editable}>
        <Widgets.SlidersWidget widget={widget}
                               label={this.getLabel(widget)}
                               min={meta.min}
                               max={meta.max}
                               proportionally={meta.proportionally}
                               value={this.getValue(widget)}
                               onChange={this.onPropertyAxisValueChange}
                               onPrecisionModal={property => this.setState({ precisionModal: { widget, property } })}
                               onProportionallyToggle={this.onProportionallyToggle} />
      </DraggableWidget>
    );
  }

  renderColorPickerList = (widget: IColorPickerList, index: number, path: string) => {
    const { droppable, selected, editable, dragging } = this.props;

    const selection = `${path}_${index}_${WidgetTypes.ColorPickerList}`;
    const draggableId = `${widget.id}_${WidgetTypes.ColorPickerList}` || tempId;
    const droppableId = this.props.getDroppableId(widget.id, `${path}[${index}]items`, [WidgetTypes.ColorPicker, WidgetTypes.MiniColorPicker], 'COLORS_LIST') || tempId;
    const isDropDisabled = (droppable !== droppableId);

    return (
      <DraggableWidget draggableId={draggableId}
                       index={index}
                       className="color-pickers-list"
                       onSelect={this.props.onSelected.bind(this, selection)}
                       selected={this.props.selected === selection}
                       key={draggableId}
                       isDragDisabled={!editable}>
        <DroppableWidget key={droppableId}
                         droppableId={droppableId}
                         className="multiline-widget-wrapper"
                         isDropDisabled={isDropDisabled}>
          <ColorPickerList list={widget}
                           path={`${path}[${index}]items`}
                           selection={selected}
                           dragging={dragging}
                           isDragDisabled={!editable}
                           onSelect={this.props.onSelected}
                           getPropertyValue={this.getPropertyValue}
                           getPropertyLabel={this.getLabel}
                           getAlpha={this.getAlpha}
                           onChange={this.onThrottledPropertyValueChange}
                           onPrecisionModal={widget => this.setState({ precisionModal: { widget } })} />
        </DroppableWidget>
      </DraggableWidget>
    );
  }

  renderColorList = (widgets: ICustomStackProperty[], listIndex: number, path: string) => {
    const { droppable, preset } = this.props;
    const first = _.first(widgets);

    if (!first)
      return null;

    const property = preset.Exposed[first.property];
    const type = property?.Metadata?.Widget ?? first.widget;
    const droppableId = this.props.getDroppableId(first.id, path, [type], 'COLOR_LIST') || tempId;
    const isDropDisabled = (droppable !== droppableId);

    return (
      <DraggableWidget draggableId={first.id || tempId}
                       index={listIndex}
                       isDragDisabled
                       className="colors-list"
                       key={first.id || tempId}>
        <DroppableWidget key={droppableId}
                         droppableId={droppableId}
                         direction="horizontal"
                         className="multiline-widget-wrapper"
                         type={type as WidgetTypes}
                         isDropDisabled={isDropDisabled}>
          {widgets.map((widget, index) => {
            const draggableId = `${widget.id}_${type}` || tempId;
            const value = this.getValue(widget) as ColorProperty | VectorProperty;
            const selection = `${path}_${index + listIndex}_${widget.property}`;

            return (
              <DraggableWidget key={draggableId}
                               draggableId={draggableId}
                               index={index + listIndex}
                               className="color"
                               onPointerChange={this.onDragChange}
                               onSelect={this.props.onSelected.bind(this, selection)}
                               selected={this.props.selected === selection}
                               isDragDisabled={!this.props.editable}>
                <ColorPicker value={value}
                             label={this.getLabel(widget)}
                             type={widget.propertyType}
                             alpha={this.getAlpha(widget.property)}
                             widget={property?.Metadata?.Widget ?? widget.widget}
                             onChange={value => this.onThrottledPropertyValueChange(widget, value)}
                             onPrecisionModal={() => this.setState({ precisionModal: { widget } })} />
              </DraggableWidget>
            );
          })}
        </DroppableWidget>
      </DraggableWidget>
    );
  }

  renderTabs = (widget: ICustomStackTabs, index: number, path: string) => {
    const { droppable } = this.props;
    const selection = { ...this.state.selection };
    const options = widget.tabs.map(tab => ({ value: tab.id, label: tab.label }));
    const firstTab = _.first(widget.tabs);

    let selected = selection[widget.id];
    let tab = _.find(widget.tabs, ['id', selected]);

    if (!tab) {
      selected = firstTab?.id;
      tab = firstTab;
    }

    const onSelect = (id: string) => this.onSelect(widget.id, id);
    const draggableId = this.getWidgetId(widget, index);
    this.props.getDroppableId(widget.id, `${path}[${index}]`, [], 'TABS');

    return (
      <DraggableWidget draggableId={draggableId}
                       key={draggableId}
                       index={index}
                       onSelect={this.props.onSelected.bind(this, `${path}_${index}_PROPERTY`)}
                       selected={this.props.selected === `${path}_${index}_PROPERTY`}
                       className="tab-group"
                       isDragDisabled>
        <Widgets.TabWidget options={options}
                          onSelect={onSelect}
                          prefix={widget.id}
                          value={selected} />
        <div style={{ width: '100%', position: 'relative' }}>
          {widget.tabs.map((w, i) => {
            const droppableId = this.props.getDroppableId(w.id, `${path}[${index}]tabs[${i}]widgets`,
              [
                'WIDGET',
                WidgetTypes.Dial,
                WidgetTypes.Button,
                WidgetTypes.Toggle,
                WidgetTypes.ColorPicker,
                WidgetTypes.MiniColorPicker,
                WidgetTypes.ColorPickerList,
                WidgetTypes.Label,
                WidgetTypes.Spacer
              ], WidgetTypes.Tabs);
            const isSelected = w.id === tab.id;

            let isDropDisabled = false;
            let className = 'droppable-list ';

            if (!isSelected)
              className += 'hidden';

            if (droppable !== droppableId)
              isDropDisabled = true;

            return (
              <DroppableWidget key={droppableId}
                               droppableId={droppableId}
                               className={className}
                               isDropDisabled={isDropDisabled}>
                {this.renderWidgets(w.widgets, `${path}[${index}]tabs[${i}]widgets`)}
              </DroppableWidget>
            );
          })}
        </div>
      </DraggableWidget>
    );
  }

  renderListItemContent = (item: ICustomStackListItem, index: number, isSelected: boolean) => {
    const { droppable, editable } = this.props;
    const path = `[0]items[${index}]panels`;
    const droppableId = this.props.getDroppableId(item.id, path,
      [
        'WIDGET',
        'PANEL',
        WidgetTypes.Button,
        WidgetTypes.Toggle,
        WidgetTypes.ColorPicker,
        WidgetTypes.MiniColorPicker,
        WidgetTypes.Tabs,
        WidgetTypes.Dial,
        WidgetTypes.Label,
        WidgetTypes.Spacer,
        WidgetTypes.ColorPickerList,
        WidgetTypes.Dropdown,
      ], 'LIST');

    let isDropDisabled = false;
    let className = 'droppable-root ';

    if (!isSelected)
      className += 'hidden ';

    if (droppable !== droppableId)
      isDropDisabled = true;

    if (!editable)
      return <VirtualList droppableId={droppableId}
                          data={item.panels}
                          className={className}
                          isDropDisabled={isDropDisabled}
                          itemContent={(index, panel) => this.renderPanel('', panel, index)} />;

    return (
      <DroppableWidget droppableId={droppableId}
                       key={item.id}
                       className={className}
                       isDropDisabled={isDropDisabled}>
        {item.panels?.map((panel, index) => this.renderPanel(path, panel, index))}
      </DroppableWidget>
    );

  }

  renderList = (list: IPanel) => {
    const { editable } = this.props;
    const selection = { ...this.state.selection };
    const listId = list.id;
    const firstListItem = _.first(list?.items);

    let selected = selection[list.id];
    let selectedItem = _.find(list.items, ['id', selected]);

    if (!selectedItem) {
      selected = firstListItem?.id;
      selectedItem = firstListItem;
    }

    const onSelect = (id: any) => {
      selection[list.id] = id;
      this.setState({ selection });
    };

    this.props.getDroppableId(listId, `0`, [], 'LIST');

    return (
      <DraggableWidget key={listId}
                       draggableId={listId}
                       index={0}
                       className="navigation-list"
                       onSelect={this.props.onSelected.bind(this, '_0')}
                       selected={this.props.selected === '_0'}
                       isDragDisabled>
        <div className="navigation">
          {list.items?.map((item, index) => {
            const itemId = item.id;

            return (
              <div className="navigation-row" 
                   data-prefix={list.id} 
                   data-value={itemId}
                   key={itemId}>
                <button className={`navigation-btn ${selected === item.id ? 'selected' : ''}`} 
                        onClick={() => onSelect(item.id)}><p>{item.label}</p></button>
              </div>
            );
          })}
        </div>
        <div className="navigation-list-panels">
          {editable &&
            list.items.map((item, index) => this.renderListItemContent(item, index, item.id === selectedItem.id))
          }
          {!editable &&
            this.renderListItemContent(selectedItem, 0, true)
          }
        </div>
      </DraggableWidget>
    );
  }

  renderDropdown = (widget: ICustomStackProperty, index: number, path: string) => {
    const id = this.getWidgetId(widget, index);
    const property = this.props.preset.Exposed[widget.property] as IExposedProperty;
    const options = property?.UnderlyingProperty?.Metadata?.EnumValues?.split(',').map(value => value.trim()) ?? [];
    const selection = `${path}_${index}_${widget.property}`;

    return (
      <DraggableWidget draggableId={id}
                       index={index}
                       key={id}
                       className="dropdown"
                       onPointerChange={this.onDragChange}
                       onSelect={this.props.onSelected.bind(this, selection)}
                       selected={this.props.selected === selection}
                       isDragDisabled={!this.props.editable}>
        <Widgets.DropdownWidget options={options}
                                value={this.getValue(widget)}
                                label={this.getLabel(widget)}
                                onChange={value => this.onPropertyValueChange(widget, value)} />
      </DraggableWidget>
    );
  }

  renderSpacerWidget = (widget: ICustomStackProperty, index: number, path: string) => {
    const id = this.getWidgetId(widget, index);
    const height = (widget?.spaces || 1) * 10 + 'px';
    const selection = `${path}_${index}_${widget.property}`;

    return (
      <DraggableWidget draggableId={id}
                       index={index}
                       key={id}
                       onPointerChange={this.onDragChange}
                       onSelect={this.props.onSelected.bind(this, selection)}
                       selected={this.props.selected === selection}
                       isDragDisabled={!this.props.editable}>
        <div style={{ height }} />
      </DraggableWidget>
    );
  }

  renderImageSelector = (widget: ICustomStackProperty, index: number, path: string) => {
    const id = this.getWidgetId(widget, index);

    return (
      <DraggableWidget key={id}
                       draggableId={id}
                       index={index}
                       className="image-selector"
                       onPointerChange={this.onDragChange}
                       onSelect={this.props.onSelected.bind(this, `${path}_${index}`)}
                       selected={this.props.selected === `${path}_${index}`}
                       isDragDisabled={!this.props.editable}>
        <Widgets.ImageSelectorWidget widget={widget}
                             value={this.getValue(widget)}
                             onChange={this.onPropertyValueChange} />
      </DraggableWidget>
    );
  }

  renderToggleWidgets = (widgets: ICustomStackProperty[], index: number, path: string) => {
    const { droppable } = this.props;
    const toggle = _.first(widgets);

    if (!toggle.id)
      return null;

    const draggableId = `${WidgetTypes.Toggle}_${toggle.id}`;
    const droppableId = this.props.getDroppableId(toggle.id, path, [WidgetTypes.Toggle], 'TOGGLES');

    let isDropDisabled = false;

    if (droppable !== droppableId)
      isDropDisabled = true;

    return (
      <DraggableWidget key={draggableId}
                       draggableId={draggableId}
                       index={index}
                       className="toggles-list"
                       isDragDisabled>
        <DroppableWidget droppableId={droppableId}
                         direction="horizontal"
                         className="multiline-widget-wrapper"
                         type={WidgetTypes.Toggle}
                         isDropDisabled={isDropDisabled}>
          {widgets.map((widget, i) => {
            const draggableId = `${widget.id}_${WidgetTypes.Toggle}`;
            const selection = `${path}_${index + i}_${widget.property}`;

            return (
              <DraggableWidget key={draggableId + i}
                               draggableId={draggableId}
                               index={index + i}
                               className="toggle"
                               onSelect={this.props.onSelected.bind(this, selection)}
                               selected={this.props.selected === selection}
                               isDragDisabled={!this.props.editable}>
                <Widgets.ToggleWidget widget={widget}
                              checked={!!this.getValue(widget)}
                              label={this.getLabel(widget)}
                              onChange={this.onPropertyValueChange} />
              </DraggableWidget>
            );
          })}
        </DroppableWidget>
      </DraggableWidget>
    );
  }

  renderLabel = (widget: ICustomStackProperty, index: number, path: string) => {
    const id = this.getWidgetId(widget, index);
    const selection = `${path}_${index}_${widget.property}`;

    return (
      <DraggableWidget key={id}
                       draggableId={id}
                       index={index}
                       className="label"
                       onPointerChange={this.onDragChange}
                       onSelect={this.props.onSelected.bind(this, selection)}
                       selected={this.props.selected === selection}
                       isDragDisabled={!this.props.editable}>
        {widget.label}
      </DraggableWidget>
    );
  }

  renderTextWidget = (widget: ICustomStackProperty, index: number, path: string) => {
    const onChange = this.onPropertyValueChange.bind(this, widget);
    const id = this.getWidgetId(widget, index);
    const selection = `${path}_${index}_${widget.property}`;

    return (
      <DraggableWidget className="slider-row text-row"
                       key={id}
                       index={index}
                       draggableId={id}
                       onPointerChange={this.onDragChange}
                       onSelect={this.props.onSelected.bind(this, selection)}
                       selected={this.props.selected === selection}
                       isDragDisabled={!this.props.editable}>
        <div className="title">{this.getLabel(widget)}</div>
        <Text value={this.getValue(widget)} onChange={onChange} />
        <FontAwesomeIcon icon={['fas', 'undo']} onClick={onChange} />
      </DraggableWidget>
    );
  }

  renderAssetWidget = (widget: ICustomStackProperty, index: number, path: string) => {
    const { preset } = this.props;
    const id = this.getWidgetId(widget, index);
    const selection = `${path}_${index}_${widget.property}`;

    const exposed = preset?.Exposed[widget.property] as IExposedProperty;

    return (
      <DraggableWidget className="asset-row"
                       key={id}
                       index={index}
                       draggableId={id}
                       onPointerChange={this.onDragChange}
                       onSelect={this.props.onSelected.bind(this, selection)}
                       selected={this.props.selected === selection}
                       isDragDisabled={!this.props.editable}>
        <Widgets.AssetWidget label={this.getLabel(widget)}
                             value={this.getValue(widget)}
                             type={exposed?.Type ?? 'Object'}
                             typePath={exposed?.TypePath}
                             onChange={value => this.onPropertyValueChange(widget, value)}
                             browse />
      </DraggableWidget>
    );
  }

  renderDialsWidget = (widget: ICustomStackProperty, index: number, path: string) => {
    const id = this.getWidgetId(widget, index);    
    const selection = `${path}_${index}_${widget.property}`;

    const rotator = widget.propertyType === PropertyType.Rotator;
    const { min, max, proportionally } = this.getWidgetMetadata(widget);

    let dialMode = DialMode.Endless;

    if (min !== undefined && max !== undefined)
      dialMode = DialMode.Range;

    if (rotator)
      dialMode = DialMode.Rotation;
      
    return (
      <DraggableWidget key={id}
                       draggableId={id}
                       index={index}
                       className="vector-dials"
                       onPointerChange={this.onDragChange}
                       onSelect={this.props.onSelected.bind(this, selection)}
                       selected={this.props.selected === selection}
                       isDragDisabled={!this.props.editable}>
        <DialsWrapper type={widget.propertyType}
                      min={min}
                      max={max}
                      proportionally={proportionally}
                      property={widget.property}
                      mode={dialMode}
                      hidePrecision={true}
                      properties={widget?.widgets}
                      value={this.getValue(widget)}
                      label={this.getLabel(widget)}
                      onChange={value => this.onThrottledPropertyValueChange(widget, value)}
                      onProportionallyToggle={this.onProportionallyToggle} />
      </DraggableWidget>
    );
  }

  renderDialWidgets = (widgets: ICustomStackProperty[], index: number, path: string) => {
    const { droppable } = this.props;
    const dial = _.first(widgets);

    if (!dial?.id)
      return null;

    const draggableId = `${WidgetTypes.Dial}_${dial.id}`;
    const droppableId = this.props.getDroppableId(dial.id, path, [WidgetTypes.Dial], 'DIALS');

    let isDropDisabled = false;

    if (droppable !== droppableId)
      isDropDisabled = true;
      
    return (
      <DraggableWidget key={draggableId}
                       draggableId={draggableId}
                       index={index}
                       className="dials-list"
                       isDragDisabled>
        <DroppableWidget droppableId={droppableId}
                         direction="horizontal"
                         className="multiline-widget-wrapper"
                         type={WidgetTypes.Dial}
                         isDropDisabled={isDropDisabled}>
          {widgets.map((widget, i) => {
            const draggableId = `${widget.id}_${WidgetTypes.Dial}`;
            const meta = this.getWidgetMetadata(widget);
            const mode = (meta.min !== undefined && meta.max !== undefined) ? DialMode.Range : DialMode.Endless;
            const selection = `${path}_${index + i}_${widget.property}`;

            return (
              <DraggableWidget key={draggableId + i}
                               draggableId={draggableId}
                               index={index + i}
                               className="dial"
                               onSelect={this.props.onSelected.bind(this, selection)}
                               selected={this.props.selected === selection}
                               isDragDisabled={!this.props.editable}>
                <Dial value={this.getValue(widget)}
                      label={this.getLabel(widget)}
                      onChange={value => this.onThrottledPropertyValueChange(widget, value)}
                      min={meta.min}
                      max={meta.max}
                      mode={mode}
                      onPrecisionModal={() => this.setState({ precisionModal: { widget } })} />
              </DraggableWidget>
            );
          })}
        </DroppableWidget>
      </DraggableWidget>
    );
  }

  onPropertyAxisValueChange = (widget: ICustomStackProperty, axis: string, axisValue: number, proportionally?: boolean, min?: number, max?: number) => {
    const value = { ...this.getValue(widget) } ?? {};

    // Reset
    if (axis === undefined || axisValue === undefined)
      return this.onPropertyValueChange(widget);

    let prev = value[axis];
    if (prev === 0 || prev === undefined)
      prev = 1;

    const ratio = Math.max(0.001, axisValue) / prev;

    if (proportionally && !isNaN(ratio))
      for (const key of Object.keys(value)) {
        let val = value[key] * ratio;
        if (!isNaN(min) && !isNaN(max))
          val = Math.min(max, Math.max(min, val));

        value[key] = val;
      }
    else
      value[axis] = axisValue;

    this.onThrottledPropertyValueChange(widget, value);
  }

  onToggleVectorDrawer = (vector: ICustomStackProperty) => {
    if (this.props.vector?.id === vector.id)
      vector = null;

    this.props.onSetVectorDrawer(vector);
  }

  onMetadataChange = (property: string, key: string, value: string) => {
    _api.payload.metadata(property, key, value);
  }

  onProportionallyToggle = (property: string, value: string) => {
    this.onMetadataChange(property, 'Proportionally', value);
  }

  renderVectorWidget = (widget: ICustomStackProperty, index: number, path: string) => {
    const { vector } = this.props;
    const id = this.getWidgetId(widget, index);
    const meta = this.getWidgetMetadata(widget);
    const selection = `${path}_${index}_${widget.property}`;

    return (
      <DraggableWidget key={id}
                       draggableId={id}
                       index={index}
                       className="vector"
                       onPointerChange={this.onDragChange}
                       onSelect={this.props.onSelected.bind(this, selection)}
                       selected={this.props.selected === selection}
                       isDragDisabled={!this.props.editable}>
        <Widgets.VectorWidget widget={widget}
                              label={this.getLabel(widget)}
                              value={this.getValue(widget)}
                              vector={vector}
                              min={meta.min}
                              max={meta.max}
                              proportionally={meta.proportionally}
                              onAxisChange={this.onPropertyAxisValueChange}
                              onToggleVectorDrawer={this.onToggleVectorDrawer}
                              onSetVector={this.props.onSetVectorDrawer}
                              onProportionallyToggle={this.onProportionallyToggle} />
      </DraggableWidget>
    );
  }

  renderButtonWidgets = (widgets: ICustomStackProperty[], index: number, path: string) => {
    const { droppable } = this.props;
    const button = _.first(widgets);

    if (!button.id)
      return null;

    const draggableId = `${WidgetTypes.Button}_${button.id}`;
    const droppableId = this.props.getDroppableId(button.id, path, [WidgetTypes.Button], 'BUTTONS');

    let isDropDisabled = false;

    if (droppable !== droppableId)
      isDropDisabled = true;

    return (
      <DraggableWidget key={draggableId}
                       draggableId={draggableId}
                       index={index}
                       className="buttons-list"
                       isDragDisabled>
        <DroppableWidget droppableId={droppableId}
                         direction="horizontal"
                         className="multiline-widget-wrapper"
                         type={WidgetTypes.Button}
                         isDropDisabled={isDropDisabled}>
          {widgets.map((widget, i) => {
            const draggableId = `${widget.id}_${WidgetTypes.Button}`;
            const selection = `${path}_${index + i}_${widget.property}`;

            return (
              <DraggableWidget key={draggableId + i}
                               draggableId={draggableId}
                               index={index + i}
                               className="btn-wrapper"
                               onSelect={this.props.onSelected.bind(this, selection)}
                               selected={this.props.selected === selection}
                               isDragDisabled={!this.props.editable}>
                <Button label={this.getLabel(widget)} onExecute={() => this.onPropertyValueChange(widget)} />
              </DraggableWidget>
            );
          })}
        </DroppableWidget>
      </DraggableWidget>
    );
  }

  renderJoysticks = (widget: ICustomStackProperty, index: number, path: string) => {
    const id = this.getWidgetId(widget, index);
    const keys = WidgetUtilities.getPropertyKeys(widget?.propertyType);
    const meta = this.getWidgetMetadata(widget);
    const selection = `${path}_${index}_${widget.property}`;

    return (
      <DraggableWidget key={id}
                       index={index}
                       draggableId={id}
                       onPointerChange={this.onDragChange}
                       onSelect={this.props.onSelected.bind(this, selection)}
                       selected={this.props.selected === selection}
                       isDragDisabled={!this.props.editable}
                       className="joystick-group">
        <div className="joystick-group">
          <JoysticksWrapper type={PropertyType.Vector}
                            keys={keys}
                            value={this.getValue(widget)}
                            min={meta.min}
                            max={meta.max}
                            label={this.getLabel(widget)}
                            onChange={v => this.onThrottledPropertyValueChange(widget, v)} />
        </div>
      </DraggableWidget>
    );
  }

  getInlineWidgets = (elements: React.ReactNode[], widgets: ICustomStackWidget[], type: WidgetTypes, index: number, limit: number) => {
    const w = [];

    for (let i = 0; i < limit; i++) {
      if (this.getWidgetType(widgets[index] as ICustomStackProperty) !== type)
        break;

      const draggableId = `${index}_${widgets[index].id}`;
      elements.push(<DraggableWidget key={draggableId} draggableId={draggableId} index={index} />);

      w.push(widgets[index++] as ICustomStackProperty);
    }

    return w;
  }

  getWidgetType = (widget: ICustomStackProperty) => {
    const { preset } = this.props;
    const property = preset?.Exposed[widget?.property];
    return property?.Metadata?.Widget || widget?.widget;
  }

  renderWidgets = (widgets: ICustomStackWidget[], path: string) => {
    const elements = [];
    let firstIndex = null;
    let w = [];

    for (let index = 0; index < widgets.length; index++) {
      const widget = widgets[index] as ICustomStackProperty;
      const widgetType = this.getWidgetType(widget);
      widget.widget = widgetType;

      let element = null;

      switch (widgetType) {
        case WidgetTypes.Tabs:
          element = this.renderTabs(widgets[index] as ICustomStackTabs, index, path);
          break;

        case WidgetTypes.Sliders:
          element = this.renderSliders(widget, index, path);
          break;

        case WidgetTypes.ImageSelector:
          element = this.renderImageSelector(widget, index, path);
          break;

        case WidgetTypes.Spacer:
          element = this.renderSpacerWidget(widget, index, path);
          break;

        case WidgetTypes.Joystick:
          element = this.renderJoysticks(widget, index, path);
          break;

        case WidgetTypes.Dropdown:
          element = this.renderDropdown(widget, index, path);
          break;

        case WidgetTypes.Label:
          element = this.renderLabel(widget, index, path);
          break;

        case WidgetTypes.Slider:
          element = this.renderSlider(widget, index, path);
          break;

        case WidgetTypes.Text:
          element = this.renderTextWidget(widget, index, path);
          break;

        case WidgetTypes.Vector:
          element = this.renderVectorWidget(widget, index, path);
          break;

        case WidgetTypes.Dials:
          element = this.renderDialsWidget(widget, index, path);
          break;

        case WidgetTypes.Asset:
          element = this.renderAssetWidget(widget, index, path);
          break;

        case WidgetTypes.Dial:
          firstIndex = index;
          w = this.getInlineWidgets(elements, widgets, WidgetTypes.Dial, index, 3);

          index += Math.max(w.length - 1, 0);

          elements.push(this.renderDialWidgets(w, firstIndex, path));
          break;

        case WidgetTypes.Toggle: 
          firstIndex = index;
          w = this.getInlineWidgets(elements, widgets, WidgetTypes.Toggle, index, 4);

          index += Math.max(w.length - 1, 0);

          elements.push(this.renderToggleWidgets(w, firstIndex, path));
          break;

        case WidgetTypes.Button: 
          firstIndex = index;
          w = this.getInlineWidgets(elements, widgets, WidgetTypes.Button, index, 4);

          index += Math.max(w.length - 1, 0);

          elements.push(this.renderButtonWidgets(w, firstIndex, path));
          break;


        case WidgetTypes.ColorPicker:
          firstIndex = index;
          w = this.getInlineWidgets(elements, widgets, WidgetTypes.ColorPicker, index, 3);

          index += Math.max(w.length - 1, 0);

          elements.push(this.renderColorList(w, firstIndex, path));
          break;

        case WidgetTypes.MiniColorPicker:
          firstIndex = index;
          w = this.getInlineWidgets(elements, widgets, WidgetTypes.MiniColorPicker, index, 3);

          index += Math.max(w.length - 1, 0);

          elements.push(this.renderColorList(w, firstIndex, path));
          break;

        case WidgetTypes.ColorPickerList:
          elements.push(this.renderColorPickerList(widgets[index] as IColorPickerList, index, path));
          break;

        default:
          element = <DraggableWidget draggableId={index.toString()} key={index} index={index} isDragDisabled />;
          break;
      };

      elements.push(element);
    }

    return elements;
  };

  getPanelProperties = (panel: IPanel) => {
    const ids = [];

    for (const p of panel.widgets || []) {
      const widget = p as ICustomStackProperty;

      switch (p.widget) {
        case WidgetTypes.Tabs:
          const tabs = p as ICustomStackTabs;
          for (const tab of tabs.tabs || [])
            tab.widgets.map((w: ICustomStackProperty) => ids.push(w.property));
          break;

        case WidgetTypes.ColorPickerList:
          const pickerList = p as IColorPickerList;
          for (const pickerItem of pickerList.items ?? [])
            ids.push(pickerItem.property);
          break;
  
        default:
          ids.push(widget.property);
          break;
      }
    }


    return _.compact(ids);
  }

  onRefreshPanelOptions = async (panel: IPanel) => {
    const { preset } = this.props;

    const PropertyIds = this.getPanelProperties(panel);
    if (!PropertyIds.length)
      return;

    const matching = await _api.proxy.function(
      '/Script/RemoteControlWebInterface.Default__RCWebInterfaceBlueprintLibrary',
      'FindMatchingActorsToRebind',
      {
        PresetId: preset.ID,
        PropertyIds
      });

    
    const selected = await _api.proxy.function(
      '/Script/RemoteControlWebInterface.Default__RCWebInterfaceBlueprintLibrary',
      'GetOwnerActorLabel',
      {
        PresetId: preset.ID,
        PropertyIds
      });


    const templates = { ...this.state.templates };
    templates[panel.id] = { matching, selected };

    this.setState({ templates });
    sessionStorage.setItem('templates', JSON.stringify(templates));
  }

  onRebindActor = (panel: IPanel, selected: string) => {
    const templates = { ...this.state.templates };
    const template = templates[panel.id];
    if (!template)
      return;

    const owner = template.matching[selected];
    if (!owner)
      return;

    const ids = this.getPanelProperties(panel);
    _api.payload.rebind(ids, owner);

    template.selected = selected;
    this.setState({ templates });
    sessionStorage.setItem('templates', JSON.stringify(templates));
  }

  renderPanel = (path: string, panel: IPanel, index: number,) => {
    const { editable, droppable } = this.props;

    if (!panel.id)
      return null;

    switch (panel.type) {
      case IPanelType.Panel:
        const draggableId = `${panel.id}_PANEL`;
        const panelPath = `${path}[${index}]widgets`;
        const droppableId = this.props.getDroppableId(draggableId, panelPath,
          [
            'WIDGET',
            WidgetTypes.Button,
            WidgetTypes.Toggle,
            WidgetTypes.ColorPicker,
            WidgetTypes.MiniColorPicker,
            WidgetTypes.ColorPickerList,
            WidgetTypes.Tabs,
            WidgetTypes.Dial,
            WidgetTypes.Dials,
            WidgetTypes.Label,
            WidgetTypes.Spacer,
            WidgetTypes.Dropdown,
          ], 'PANEL');

        let isDropDisabled = false;

        if (droppable !== droppableId)
          isDropDisabled = true;

        const template = this.state.templates[panel.id];
        return (
          <DraggableWidget draggableId={draggableId}
                           key={draggableId}
                           index={index}
                           className={`draggable-panel`}
                           onSelect={this.props.onSelected.bind(this, `${path}_${index}`)}
                           selected={`${this.props.selected}` === `${path}_${index}`}
                           isDragDisabled={!editable}>
            <DroppableWidget droppableId={droppableId}
                             className={`droppable-panel ${!!panel?.title || !!panel?.isTemplate ? 'has-title' : ''}`}
                             isDropDisabled={isDropDisabled}>
              {!!panel?.title && <div className="title">{panel.title}</div>}
              {!!panel.isTemplate && (
                <div className="title template-property-select-wrapper">
                  <div className='template-property-select-inner-wrapper'>
                    <select className="dropdown template-property-select" 
                            tabIndex={-1}
                            onFocus={this.isIos ? null : () => this.onRefreshPanelOptions(panel)}
                            value={template?.selected ? template?.selected : undefined} 
                            onChange={e => this.onRebindActor(panel, e.target.value)}>
                      {Object.keys(template?.matching ? template?.matching :{'Click to Select Actor': ''} ).map(match =>
                        <option key={match} value={match}>{match}</option>
                      )}
                    </select>
                  </div>
                    {this.isIos && 
                      <button onClick={() => this.onRefreshPanelOptions(panel)}><FontAwesomeIcon icon={['fas', 'sync-alt']} /></button>
                    }
                </div>
              )}
              {this.renderWidgets(panel.widgets, panelPath)}
            </DroppableWidget>
          </DraggableWidget>
        );

      case IPanelType.List:
        return this.renderList(panel);

      default:
        return null;
    }
  }

  renderVectorDrawer = () => {
    const { vector } = this.props;
    if (!vector)
      return null;

    const meta = this.getWidgetMetadata(vector);
    if (!meta.type)
      return null;

    return (
      <VectorDrawer widget={vector}
                    label={this.getLabel(vector)}
                    value={this.getValue(vector)}
                    min={meta.min}
                    max={meta.max}
                    onClose={this.props.onSetVectorDrawer}
                    onChange={value => this.onThrottledPropertyValueChange(vector, value)} />
    );
  }

  renderRootList = () => {
    const { editable, droppable, panels = [], tabKey } = this.props;
    const isList = panels[0]?.type === IPanelType.List;
    const droppableId = this.props.getDroppableId(`${tabKey}_ROOT`, '', ['ALL'], 'ROOT');

    let isDropDisabled = false;
    if (droppable !== droppableId || isList)
      isDropDisabled = true;

    if (editable)
      return (
        <DroppableWidget droppableId={droppableId} 
                         className="droppable-root" 
                         isDropDisabled={isDropDisabled}>
          {panels?.map((panel, index) => this.renderPanel('', panel, index))}
        </DroppableWidget>
      );

    return <VirtualList droppableId={droppableId}
                        data={panels}
                        className="droppable-root"
                        isDropDisabled={isDropDisabled}
                        itemContent={(index, panel) => this.renderPanel('', panel, index)} />;
  }

  render() {
    const { dragging, panels = [], vector } = this.props;
    const isList = panels[0]?.type === IPanelType.List;
    let className = 'customstack-wrapper ';

    if (!!dragging)
      className += 'drag ';

    if (isList)
      className += 'customstack-list ';

    if (!!vector)
      className += 'drawer-shown ';

    return (
      <div className={className}>
        {this.renderRootList()}
        {this.renderVectorDrawer()}
        {this.renderPrecisionModal()}
      </div>
    );
  }
}