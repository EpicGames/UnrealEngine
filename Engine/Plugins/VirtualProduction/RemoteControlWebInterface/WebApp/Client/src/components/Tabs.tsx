import React from 'react';
import { ITab, IPreset } from '../shared';
import { _api } from '../reducers';
import { FontAwesomeIcon } from '@fortawesome/react-fontawesome';
import { IconName } from '@fortawesome/fontawesome-svg-core';
import { Droppable, Draggable, DraggableStateSnapshot } from 'react-beautiful-dnd';


type State = {
  allTabsVisible: boolean;
};

type Props = {
  preset: IPreset;
  tabs: ITab[];
  selected: number;
  editable?: boolean;
  locked: boolean;
  undo: boolean;
  redo: boolean;

  onLockedChange: () => void;
  onChange: (index: number) => void;
  onNewTab: () => void;
  onDeleteTab: (tab: number) => Promise<void>;
};

export class Tabs extends React.Component<Props> {
  ref = React.createRef<HTMLDivElement>();
  state: State = { 
    allTabsVisible: true,
  };

  componentDidMount() {
    window.addEventListener('resize', this.onResize);
  }

  componentDidUpdate(prevProps: Props) {
    if (prevProps.tabs !== this.props.tabs || 
        prevProps.selected !== this.props.selected ||
        prevProps.editable !== this.props.editable) 
      this.onResize();
  }

  componentWillUnmount() {
    window.removeEventListener('resize', this.onResize);
  }

  setRef = (innerRef: (el) => void, el) => {
    this.ref = { current: el };

    innerRef(el);
  }

  getStyle = (style: React.CSSProperties, snapshot: DraggableStateSnapshot): React.CSSProperties => {
    let transform = style.transform;
    transform = transform?.replace(/(, (-|)[0-9]+)/, ', 0') || '';    

    return { ...style, transform };
  }

  onResize = () => {
    const wrapper = this.ref?.current;
    if (!wrapper)
      return;

    const { children } = wrapper;
    let childrenWidth = 0;
    for (let el of children)
      childrenWidth += el.clientWidth;
    
    this.setState({ allTabsVisible: childrenWidth < wrapper.clientWidth });
    this.onTabChange(this.props.selected);
  }

  onTabChange = (index: number): void => {
    const wrapper = this.ref.current;
    if (index < 0 || index >= this.props.tabs?.length || !wrapper)
      return;
      
    if (this.props.selected !== index)
      this.props.onChange?.(index);

    const { children, scrollLeft, clientWidth } = wrapper;

    let offsetStart = 0;
    for (let i = 0; i < index; i++)
      offsetStart += children[i]?.clientWidth ?? 0;

    const offsetEnd = offsetStart + children[index]?.clientWidth;
    const visibleLeft = scrollLeft <= offsetStart;
    const visibleRight = offsetEnd <= scrollLeft + clientWidth;
    
    if (!visibleRight)
      wrapper.scrollTo(offsetEnd - clientWidth, 0);

    if (!visibleLeft) 
      wrapper.scrollTo(offsetStart, 0);
  }

  renderTab = (tab: ITab, index: number) => {
    const { editable, selected } = this.props;
    const id = `tab-${index}`;

    return(
      <Draggable key={id} 
                 draggableId={id}
                 index={index}
                 isDragDisabled={!editable}>
        {(provided, snapshot) => (
          <div className={`tab ${index === selected && 'selected'}`}
               ref={provided.innerRef}
               {...provided.draggableProps}
               {...provided.dragHandleProps}
               style={{ ...this.getStyle(provided.draggableProps.style, snapshot) }}
               onPointerDown={() => this.onTabChange(index)}
               tabIndex={-1}>
            <div className="icon" >
              {!!tab.icon && <FontAwesomeIcon icon={["fas", tab.icon as IconName]} />}
            </div>
            <div className="label">{tab.name}</div>
            {editable && selected === index &&
              <div className="delete-tab" onClick={this.props.onDeleteTab.bind(this, index)}><FontAwesomeIcon icon={['fas', 'times']} /></div>
            }
          </div>
        )}
      </Draggable>
    );
  }

  render() {
    const { tabs, selected, editable, preset, locked, undo, redo } = this.props;
    const { allTabsVisible } = this.state;

    let leftTabClassName = 'arrow left ';
    if (selected === 0 || allTabsVisible)
      leftTabClassName += 'disabled';

    let rightTabClassName = 'arrow right ';
    if (selected === tabs?.length - 1 || allTabsVisible)
      rightTabClassName += 'disabled ';

    let iconUndoClassName = 'icon ';
    if (!undo)
      iconUndoClassName += 'disabled';
      
    let iconRedoClassName = 'icon ';
    if (!redo)
      iconRedoClassName += 'disabled';

    return (
      <div className="tabs-wrapper">
        <div className="tabs-wrapper-inner">
          <Droppable droppableId="header-tabs" 
                     direction="horizontal" 
                     type="HEADER_TABS"
                     isDropDisabled={!editable}>
            {provided => (
              <div ref={this.setRef.bind(this, provided.innerRef)}
                   className="tabs-drop-list"
                   {...provided.droppableProps}>
                {tabs?.map(this.renderTab)}
              </div>
            )}
          </Droppable>
        </div>

        <div className="tab-navigation">
          {!!editable && 
            <div className="tab new-tab" onClick={() => this.props.onNewTab()}><FontAwesomeIcon icon={['fas', 'plus']} /></div>
          }
          <div className={leftTabClassName} onClick={() => this.onTabChange(selected - 1)}><FontAwesomeIcon icon={['fas', 'caret-left']} /></div>
          <div className={rightTabClassName} onClick={() => this.onTabChange(selected + 1)}><FontAwesomeIcon icon={['fas', 'caret-right']} /></div>
          <div className="preset-undo-redo">
            <span className={iconUndoClassName} onClick={_api.undoHistory}><FontAwesomeIcon icon={['fas', 'undo']} /></span>
            <span className={iconRedoClassName} onClick={_api.redoHistory}><FontAwesomeIcon icon={['fas', 'redo']} /></span>
          </div>
          <div className="preset-name">{preset?.Name}</div>
          <div className="tab edit-tab locked-toggle">
            <label className="switch toggle-mode locked-mode">
              <input type="checkbox" checked={!!locked} onChange={() => this.props.onLockedChange()} />
              <span className="slider inline"></span>
            </label>
          </div>
        </div>
        
      </div>
    );
  }
} 