import { FontAwesomeIcon } from '@fortawesome/react-fontawesome';
import React from 'react';
import { _api } from '../../reducers';
import { SafeImage } from '../controls';
import { SearchModal } from '../';


type Props = {
  label?: string | JSX.Element;
  value?: string;
  type?: string;
  typePath?: string;
  browse?: boolean;
  reset?: boolean;
  onChange?: (value?: string) => void;
}

export class AssetWidget extends React.Component<Props> {

  static defaultProps: Props = {
    reset: true,
  };

  onBrowse = async () => {
    let { type, typePath } = this.props;
    if (type.startsWith('U') || type.startsWith('F'))
      type = type.substring(1);

    type = type.replace('*', '');
    if (typePath?.length == 0)
      typePath = type;

    const asset = await SearchModal.open({ placeholder: 'Search asset', types: [typePath], prefix: '/Game', count: 200 });
    if (asset)
      this.props.onChange?.(asset.Path);
  }

  render() {
    const { value, label, browse, reset } = this.props;
    const url = _api.assets.thumbnailUrl(value);

    return (
      <div className="asset-item">
        <div className="label">
          {label}
        </div>
        <div className="image">
          <SafeImage src={url}
                     fallback="/images/assets/Asset.png"
                     alt="Asset"
                     className="box-image" />
        </div>
        <div className="input-title">
          <input readOnly value={value} />
          {browse &&
            <span className="asset-icon" onClick={this.onBrowse}><FontAwesomeIcon icon={['fas', 'folder']} /></span>
          }
        </div>
        {reset &&
          <FontAwesomeIcon icon={['fas', 'undo']} onClick={() => this.props.onChange?.()} />
        }
      </div>
    );
  }
};