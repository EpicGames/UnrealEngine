import React from 'react';
import { FontAwesomeIcon } from '@fortawesome/react-fontawesome';


type Props = {
  label?: React.ReactNode;
  options?: string[];
  value?: string;

  onChange?: (value?: string) => void;
}

export class DropdownWidget extends React.Component<Props> {
  onSelect = (value?: string) => {
    this.props.onChange?.(value);
  }
  
  render() {
    const { label = '', options, value } = this.props;

    return (
      <div className="dropwdown-row">
        <div className="slider-row">
          <div className="title">{label}</div>
          <div className="dropdown-widget">
            <select className="dropdown" value={value} onChange={e => this.onSelect(e.target.value)}>
              {options?.map((option, index) =>
                <option key={index} value={option}>{option}</option>
              )}
            </select>
          </div>
          <FontAwesomeIcon icon={['fas', 'undo']} onClick={() => this.props.onChange?.()} />
        </div>
      </div>
    );
  }
};