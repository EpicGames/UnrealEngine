import React from 'react';
import { ICustomStackProperty, PropertyValue } from 'src/shared';
import { SafeImage } from '../controls';
import { _api } from '../../reducers';
import _ from 'lodash';


type Props = {
  widget: ICustomStackProperty;
  value?: any;

  onChange?: (widget: ICustomStackProperty, value?: PropertyValue) => void;
}

export class ImageSelectorWidget extends React.Component<Props> {
  render() {
    const { widget, value = null } = this.props;
    const itemSize = { width: 260, height: 100 };

    return (
      <div className="image-dropdown-row" style={{ minHeight: itemSize.height + 20 }}>
        {widget.options?.map((option, index) =>
          <SafeImage key={index}
                     src={_api.assets.thumbnailUrl(option.label)}
                     className={`dropdown-image ${value === option.value ? 'selected-dropdown-image' : ''}`}
                     onClick={() => this.props.onChange?.(widget, option.value)}
                     alt={option.value}
                     label={_.startCase(_.last(widget.property.split('.')))}
                     style={itemSize} />
        )}
      </div>
    );
  }
};