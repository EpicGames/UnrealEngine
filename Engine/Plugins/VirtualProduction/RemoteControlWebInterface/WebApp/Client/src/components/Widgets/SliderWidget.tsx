import React from 'react';
import { Slider, ValueInput } from '../controls';
import { FontAwesomeIcon } from '@fortawesome/react-fontawesome';
import { PropertyValue } from 'src/shared';


type Props = {
  value?: any;
  label?: React.ReactNode;
  min?: number;
  max?: number;
  precision?: number

  onPrecisionModal?: () => void;
  onChange?: (value?: PropertyValue) => void;
}

export class SliderWidget extends React.Component<Props> {
  render() {
    const { value, label = '', min, max, precision } = this.props;
    const isRange = (min !== undefined && max !== undefined);

    return (
      <div className="slider-row">
        <div className="title">{label}</div>
        <FontAwesomeIcon icon={['fas', 'expand']} className="expand-icon" onClick={this.props?.onPrecisionModal} />
        <ValueInput min={min}
                    max={max}
                    precision={precision}
                    value={value}
                    onChange={this.props.onChange} />

        {isRange &&
          <>
            <div className="limits">{min.toFixed(1)}</div>
            <Slider value={value}
                    min={min}
                    max={max}
                    precision={precision}
                    showLabel={false}
                    onChange={this.props.onChange} />
            <div className="limits">{max.toFixed(1)}</div>
          </>
        }
        <FontAwesomeIcon icon={['fas', 'undo']} onClick={() => this.props.onChange()} />
      </div>
    );
  }
};