import React from 'react';
import { ICustomStackProperty } from 'src/shared';
import { WidgetUtilities } from 'src/utilities';
import { ValueInput, Slider } from '../controls';
import { FontAwesomeIcon } from '@fortawesome/react-fontawesome';


type Props = {
  widget: ICustomStackProperty;
  label?: React.ReactNode;
  min?: number;
  max?: number;
  proportionally?: boolean;
  value?: any;
  
  onChange?: (widget: ICustomStackProperty, axis?: string, axisValue?: number, proportionally?: boolean, min?: number, max?: number) => any;
  onPrecisionModal?: (property: string) => void;
  onProportionallyToggle?: (property: string, value: string) => void;
}

export class SlidersWidget extends React.Component<Props> {

  render() {   
    const { widget, label = '', min, max, proportionally, value } = this.props;

    const propertyType = widget?.propertyType;
    const properties = WidgetUtilities.getPropertyKeys(propertyType);
    const precision = WidgetUtilities.getPropertyPrecision(propertyType);

    const isSlider = (min !== undefined && max !== undefined);
    const selectedProperties = properties.filter(property => widget.widgets?.includes(property));

    return (
      <div className="custom-sliders">
        {properties.map(property =>
          <React.Fragment key={property}>
            {(widget.widgets?.includes(property) || !selectedProperties.length) && 
              <div className="slider-row">
                <div className="title">{label}.{property}</div>
                <FontAwesomeIcon icon={['fas', 'expand']} className="expand-icon" onClick={() => this.props.onPrecisionModal?.(property)} />
                <ValueInput min={min}
                            max={max}
                            precision={precision}
                            value={value?.[property]}
                            onChange={value => this.props.onChange?.(widget, property, value, proportionally, min, max) || null} />
                {isSlider &&
                  <>
                    <div className="limits">{min?.toFixed(1)}</div>
                    <Slider value={value?.[property] || null}
                            min={min}
                            max={max}
                            showLabel={false}
                            onChange={value => this.props.onChange?.(widget, property, value, proportionally, min, max) || null} />
                    <div className="limits">{max?.toFixed(1)}</div>
                  </>
                }
              </div>
            }
          </React.Fragment>
        )}
        <FontAwesomeIcon icon={['fas', (proportionally) ? 'lock' : 'lock-open']}
                         className='proportional icon'
                         onClick={() => this.props.onProportionallyToggle(widget.property, proportionally ? '0' : '1')} />
        <FontAwesomeIcon icon={['fas', 'undo']} className="reset-sliders" onClick={() => this.props.onChange?.(widget)} />
      </div>
    );
  }
};