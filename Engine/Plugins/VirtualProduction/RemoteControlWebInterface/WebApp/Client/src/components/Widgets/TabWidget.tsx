import React from 'react';


type Props = {
  options: { value: any, label?: string } [];
  prefix?: string;
  value?: any;

  onSelect: (value: any) => void;
};

export class TabWidget extends React.Component<Props> {

  render() {
    const { value, prefix, options, onSelect } = this.props;

    return (
      <div className="tab-widget-wrapper">
        {options.map((option, index) =>
          <button key={index}
                  onClick={onSelect.bind(this, option.value)}
                  data-prefix={prefix}
                  data-value={option.value}
                  className={`btn ${value === option.value ? 'selected' : ''}`}>
            {option.label ?? option.value}
          </button>
        )}
      </div>
    );
  }
}