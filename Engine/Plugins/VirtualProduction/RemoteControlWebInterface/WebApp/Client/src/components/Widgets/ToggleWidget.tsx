import React from 'react';
import { ICustomStackProperty, PropertyValue } from 'src/shared';


type Props = {
  widget: ICustomStackProperty;
  checked?: boolean;
  label?: React.ReactNode;

  onChange?: (widget: ICustomStackProperty, value?: PropertyValue) => void;
}

export class ToggleWidget extends React.Component<Props> {
  render() {
    const { widget, checked = false, label = '' } = this.props;

    let className = 'toggle ';
    if (checked)
      className += 'checked ';

    return (
      <div className={className}>
        <div className="title">{label}</div>
        <div className="switch" onClick={() => this.props.onChange?.(widget, !checked)} />
      </div>
    );
  }
};