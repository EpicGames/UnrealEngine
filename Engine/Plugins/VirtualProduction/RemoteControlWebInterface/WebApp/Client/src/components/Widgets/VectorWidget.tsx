import React from 'react';
import { ICustomStackProperty, WidgetTypes } from 'src/shared';
import { WidgetUtilities } from 'src/utilities';
import { ValueInput } from '../controls';
import { FontAwesomeIcon } from '@fortawesome/react-fontawesome';
import _ from 'lodash';


type Props = {
  widget: ICustomStackProperty;
  label?: React.ReactNode;
  value?: any;
  vector?: ICustomStackProperty;
  min?: number;
  max?: number;
  proportionally?: boolean;

  onAxisChange?: (widget: ICustomStackProperty, axis: string, axisValue: number, proportionally?: boolean, min?: number, max?: number) => void;
  onToggleWidgetLock?: (key: string) => void;
  onToggleVectorDrawer?: (vector: ICustomStackProperty) => void;
  onSetVector?: (vector: ICustomStackProperty) => void;
  onProportionallyToggle?: (property: string, value: string) => void;  
}

const modes = [WidgetTypes.Joystick, WidgetTypes.Dial, WidgetTypes.Sliders];

export class VectorWidget extends React.Component<Props> {

  componentDidUpdate(prevProps: Props) {
    const { widget, vector } = this.props;

    if (widget !== prevProps.widget && !!vector)
      this.props.onSetVector({ ...widget });
  }

  onToggleVectorDrawer = (e: React.MouseEvent, widget: ICustomStackProperty, key: string) => {
    e.stopPropagation();
    this.props.onToggleVectorDrawer({ ...widget });
  }
  
  render() {
    const { widget, label = '', value, vector, min, max, proportionally } = this.props;
    const key = widget.id;
    const keys = WidgetUtilities.getPropertyKeys(widget.propertyType);
    const widgets = _.compact(modes.map(k => widget.widgets?.find(w => w === k))) || [];

    return (
      <div className="slider-row joystick-row">
        <div className="title">{label}</div>

        {keys.map(property =>
          <div key={`${property}${key}`} className="value-info">
            <div className={`axis-title ${property}-axis`}>{property}:</div>
            <ValueInput min={min}
                        max={max}
                        precision={WidgetUtilities.getPropertyPrecision(widget.propertyType)}
                        value={value?.[property]}
                        onChange={value => this.props.onAxisChange?.(widget, property, value, proportionally, min, max)} />
          </div>
        )}
        { }
        <FontAwesomeIcon icon={['fas', (proportionally) ? 'lock' : 'lock-open' ]}
                         className={`proportional ${vector?.id === key ? 'selected' : ''}`}
                         onClick={() => this.props.onProportionallyToggle(widget.property, proportionally ? '0' : '1')} />

        {widget.propertyType && widgets.length > 0 &&
          <FontAwesomeIcon icon={['fas', 'gamepad']}
                           className={`gamepad ${vector?.id === key ? 'selected' : ''}`}
                           onClick={e => this.onToggleVectorDrawer?.(e, widget, key)} />
        }

        <div className="control-buttons">
          <FontAwesomeIcon icon={['fas', 'undo']} onClick={() => this.props.onAxisChange?.(widget, undefined, undefined)} />
        </div>
      </div>
    );
  }
};