import React from 'react';

type Props = {
  label?: React.ReactNode;
  onExecute?: () => void;
};

export class Button extends React.Component<Props> {

  onClick = () => {
    this.props.onExecute?.();
  }

  render() {
    return (
      <div className="widget-button-wrapper">
        <button className="widget-button" onClick={this.onClick}>
          {this.props.label || 'Button'}
        </button>
      </div>
    );
  }
}
