import React from 'react';
import { ColorProperty, VectorProperty, PropertyType, WidgetTypes } from 'src/shared';
import { ValueInput, SliderWheel } from '.';
import { FontAwesomeIcon } from '@fortawesome/react-fontawesome';
import { WidgetUtilities } from 'src/utilities';


enum SliderType {
  R  = 'R',
  G  = 'G',
  B  = 'B',
};

export type ColorPickerValue = ColorProperty | VectorProperty;

export enum ColorPickerParts {
  Circle     = 'CIRCLE',
  Wheel      = 'WHEEL',
  Value      = 'VALUE',
  Saturation = 'SATURATION',
  Alpha      = 'ALPHA',
  Inputs     = 'INPUTS',
}

export enum ColorMode {
  Rgb = 'RGB',
  Hsv = 'HSV',
}

type Props = {
  value?: ColorPickerValue;
  type?: PropertyType;
  mode?: ColorMode;
  label?: React.ReactNode;
  parts?: ColorPickerParts[];
  widget?: string;
  alpha?: boolean;
  max?: number;
  reset?: boolean;

  onChange?: (value?: ColorPickerValue) => void;
  onPrecisionModal?: () => void;
}

type State = {
  color: ColorPickerValue;
  v: number;
  max: number;
}

export class ColorPicker extends React.Component<Props, State> {

  static defaultProps: Props = {
    parts: [
      ColorPickerParts.Circle,
      ColorPickerParts.Wheel,
      ColorPickerParts.Saturation,
      ColorPickerParts.Value,
      ColorPickerParts.Alpha,
      ColorPickerParts.Inputs
    ],
    widget: WidgetTypes.ColorPicker,
    mode: ColorMode.Rgb,
    reset: true,
  };

  state: State = {
    color: null,
    v: 1,
    max: 1,
  }

  circleRef = React.createRef<HTMLDivElement>();
  rectangleRef = React.createRef<HTMLDivElement>();
  color: ColorPickerValue = null;
  monitoring: boolean = false;

  componentDidMount() {
    this.updateValues();
  }

  componentDidUpdate(prevProps: Props) {
    const { value, type, max } = this.props;

    if (prevProps.value !== value || type !== prevProps.type)
      this.updateValues();

    if (prevProps.max !== max)
      this.setState({ max });

    if (this.props.mode !== prevProps.mode)
      this.forceUpdate();
  }

  updateValues = () => {
    const { type, alpha } = this.props;
    let { max, v } = this.state;

    const color = this.getValue() as ColorProperty;
    const a = color?.A;
    delete color?.A;

    if (alpha)
      color.A = a ?? 1;

    let rgb = color;
    if (type === PropertyType.Vector4 || type === PropertyType.LinearColor) {
      max = this.props.max ?? 1;

      for (const key in color)
        if (color[key] > max)
          max = color[key];

      rgb = WidgetUtilities.colorToRgb(color, max);
    }

    const hsv = WidgetUtilities.rgb2Hsv(rgb);
    v = hsv.v || 0.001;

    this.setState({ color, max, v });
  }

  normalize = (angle: number): number => {
    while (angle < 0) 
      angle += 360;

    while (angle > 360)
      angle -= 360;

    return angle;
  }

  getValue = (): ColorPickerValue => {
    const { value, type } = this.props;

    if (value)
      return value;

    switch (type) {
      case PropertyType.Vector4:
        return { X: 1, Y: 1, Z: 1 };

      case PropertyType.Color:
      case PropertyType.LinearColor:
        return { R: 1, G: 1, B: 1 };
    }
  }

  onSetColor = (color: ColorPickerValue) => {
    this.props.onChange?.(color);
  }

  onPointerRgbDown = (e: React.PointerEvent<HTMLDivElement>) => {
    this.monitoring = true;
    this.circleRef.current.setPointerCapture(e.pointerId);

    this.onPointerRgbMove(e);
  }

  onPointerHsvDown = (e: React.PointerEvent<HTMLDivElement>) => {
    this.monitoring = true;
    this.rectangleRef.current.setPointerCapture(e.pointerId);

    this.onPointerHsvMove(e);
  }

  onPointerRgbMove = (e: React.PointerEvent<HTMLDivElement>) => {
    if (!this.monitoring)
      return;

    const { type, alpha } = this.props;
    const { v, max } = this.state;
    const rect = this.circleRef.current.getBoundingClientRect();

    const xCenter = (rect.right - rect.left) / 2 + rect.left;
    const yCenter = (rect.bottom - rect.top) / 2 + rect.top;

    const x = e.clientX - xCenter;
    const y = e.clientY - yCenter;

    const radius = Math.sqrt(x * x + y * y);
    const maxRadius = this.circleRef.current.clientHeight / 2;

    const radians = Math.atan2(x, y);
    const h = this.normalize(-1 * radians * (180 / Math.PI) - 270) / 360;

    let s = Math.min(radius / maxRadius, 1);
    if (s < 0.02)
      s = 0;

    const value = { h, s, v };
    const rgb = WidgetUtilities.hsv2rgb(value);

    if (alpha) {
      const color = this.state.color as ColorProperty;
      rgb.A = color.A;
    }

    let colorValue: ColorPickerValue = null;

    switch (type) {
      case PropertyType.Vector4:
        colorValue = WidgetUtilities.rgbToColor(rgb, max);
        break;

      case PropertyType.LinearColor:
        colorValue = WidgetUtilities.rgbToColor(rgb, max, false);
        break;

      case PropertyType.Color:
        colorValue = rgb;
        break;
    }

    this.setState({ color: colorValue });
    this.onSetColor(colorValue);
  }

  onPointerHsvMove = (e: React.PointerEvent<HTMLDivElement>) => {
    if (!this.monitoring)
      return;

    let { color, max } = this.state;
    const { type } = this.props;
    const rect = this.rectangleRef.current.getBoundingClientRect();

    const x = Math.max(0, Math.min(rect.width, e.clientX - rect.left));
    const y = Math.max(0, Math.min(rect.height, e.clientY - rect.top));

    const s = x / rect.width;
    const v = Math.max(0.01, (rect.height - y) / rect.height);

    let rgb = color as ColorProperty;
    if (type === PropertyType.LinearColor || type === PropertyType.Vector4)
      rgb = WidgetUtilities.colorToRgb(color, max);

    const hsv = WidgetUtilities.rgb2Hsv(rgb);

    hsv.s = Math.max(s, 0.0001);
    hsv.v = v;

    rgb = WidgetUtilities.hsv2rgb(hsv);

    switch (type) {
      case PropertyType.Vector4:
        color = WidgetUtilities.rgbToColor(rgb, max);
        break;

      case PropertyType.LinearColor:
        color = WidgetUtilities.rgbToColor(rgb, max, false);
        break;

      case PropertyType.Color:
        color = rgb;
        break;
    }

    this.setState({ color });
    this.onSetColor(color);
  }

  onPointerHsvUp = (e: React.PointerEvent<HTMLDivElement>) => {
    if (!this.monitoring)
      return;

    this.rectangleRef.current.releasePointerCapture(e.pointerId);
    this.monitoring = false;
  }

  onPointerRgbUp = (e: React.PointerEvent<HTMLDivElement>) => {
    if (!this.monitoring)
      return;

    this.circleRef.current.releasePointerCapture(e.pointerId);
    this.monitoring = false;
  }

  onValueInputChange = (key: string, value: number) => {
    const { color } = this.state;
    if (!color)
      return;

    color[key] = value;
    this.props.onChange?.(color);
  }

  onValueSliderChange = (property: 's' | 'v' | 'a', value: number) => {
    let { color, v, max } = this.state;
    const { type } = this.props;

    let rgb = color as ColorProperty;

    if (property === 'v')
      v = value;

    if (property === 'a' && type === PropertyType.Color)
      value *= 255;

    if (type === PropertyType.LinearColor || type === PropertyType.Vector4)
      rgb = WidgetUtilities.colorToRgb(color, max);

    const hsv = WidgetUtilities.rgb2Hsv(rgb);

    value = Math.max(value, 0.00001);
    hsv[property] = value;

    rgb = WidgetUtilities.hsv2rgb(hsv);
    color = rgb;

    switch (type) {
      case PropertyType.Vector4:
        color = WidgetUtilities.rgbToColor(color, max);
        break;

      case PropertyType.LinearColor:
        color = WidgetUtilities.rgbToColor(color, max, false);
        break;
    }

    this.setState({ color, v });
    this.onSetColor(color);
  }

  onHsvWheelMove = (sign: number) => {
    let { color, max } = this.state;
    const { type } = this.props;
    let step = sign * 0.001;

    let rgb = color as ColorProperty;
    if (type === PropertyType.LinearColor || type === PropertyType.Vector4)
      rgb = WidgetUtilities.colorToRgb(color, max);

    const hsv = WidgetUtilities.rgb2Hsv(rgb);
    if (hsv.h + step < 0)
      step += 1;

    hsv.h = hsv.h + step;

    rgb = WidgetUtilities.hsv2rgb(hsv);
    color = rgb;

    switch (type) {
      case PropertyType.Vector4:
        color = WidgetUtilities.rgbToColor(color, max);
        break;

      case PropertyType.LinearColor:
        color = WidgetUtilities.rgbToColor(color, max, false);
        break;
    }

    this.setState({ color });
    this.onSetColor(color);    
  }

  getPreviewColorStyles = (): React.CSSProperties => {
    const { color, max } = this.state;
    const { type } = this.props;

    let rgb = { ...color } as ColorProperty;

    if (type === PropertyType.LinearColor || type ===  PropertyType.Vector4)
      rgb = WidgetUtilities.colorToRgb(color, max);

    if (rgb?.A && type === PropertyType.Color)
      rgb.A /= 255;

    return { background: `rgba(${rgb?.R}, ${rgb?.G}, ${rgb?.B}, ${rgb?.A ?? 1})` };
  }

  getPointerPosition = (): { circleX?: number, circleY?: number, rectangleX?: number, rectangleY?: number } => {
    const { color, max } = this.state;
    const { type, mode } = this.props;

    let rgb: ColorProperty = null;

    switch (type) {
      case PropertyType.LinearColor:
      case PropertyType.Vector4:
        rgb = WidgetUtilities.colorToRgb(color, max);
        break;

      case PropertyType.Color:
        rgb = color as ColorProperty;
        break;
    }

    const hsv = WidgetUtilities.rgb2Hsv(rgb);
    const radian = (hsv.h - 0.25) * Math.PI * 2;

    if (mode === ColorMode.Rgb) {
      const maxCircleRadius = this.circleRef.current?.clientHeight / 2;

      const circleX = Math.sin(radian) * Math.max(0, Math.min(hsv.s, 1)) * (maxCircleRadius - 10) * -1;
      const circleY = -Math.cos(radian) * Math.max(0, Math.min(hsv.s, 1)) * (maxCircleRadius - 10) * -1;

      return { circleX, circleY };
    }

    const rectangleRef = this.rectangleRef.current;

    const rectangleWidth = rectangleRef?.clientWidth ?? 0;
    const rectangleHeight = rectangleRef?.clientHeight ?? 0;

    const rectangleX = Math.max(0, Math.min(hsv.s, 1)) * (rectangleWidth - 19);
    const rectangleY = Math.max(0, (1 - Math.max(0, hsv.v)) * (rectangleHeight - 19));
  
    return { rectangleX, rectangleY };
  }

  getVisibleParts = () => {
    const { parts, widget } = this.props;
    const visible = {} as Record<ColorPickerParts, boolean>;

    parts.map(part => visible[part] = true);

    if (widget === WidgetTypes.MiniColorPicker) {
      visible[ColorPickerParts.Circle] = false;
      visible[ColorPickerParts.Saturation] = false;
      visible[ColorPickerParts.Value] = false;
    }

    return visible;
  }

  renderRgbInput = (slider: SliderType) => {
    const { color } = this.state;
    const { type } = this.props;

    let min, max, precision = 3;
    let key: string = slider;

    switch (type) {
      case PropertyType.Color:
        precision = 0;
        min = 0;
        max = 255;
        break;

      case PropertyType.Vector:
      case PropertyType.Vector4:
        switch (slider) {
          case SliderType.R:
            key = 'X';
            break;

          case SliderType.G:
            key = 'Y';
            break;

          case SliderType.B:
            key = 'Z';
            break;
        }
        break;
    }

    let className = 'input ';
    let value = 0;
    if (color)
      value = color[key];

    switch (slider) {
      case SliderType.R:
        className += 'red ';
        break;

      case SliderType.G:
        className += 'green ';
        break;

      case SliderType.B:
        className += 'blue ';
        break;
    }

    return (
      <div className={className}>
        <ValueInput draggable={false}
                    min={min}
                    max={max}
                    precision={precision}
                    value={value}
                    onChange={this.onValueInputChange.bind(this, key)} />
      </div>
    );
  }

  rendeCircleRectangle = () => {
    const { mode, type } = this.props;
    const { color, max } = this.state;
    const pos = this.getPointerPosition();
    const style: React.CSSProperties = { transform: `translate(${pos.circleX}px, ${pos.circleY}px)` };

    if (mode === ColorMode.Rgb)
      return (
        <div className="circle">
          <div className="circle-gradient">
            <div className="canvas-area"
                 ref={this.circleRef}
                 onPointerMove={this.onPointerRgbMove}
                 onPointerDown={this.onPointerRgbDown}
                 onPointerUp={this.onPointerRgbUp}>
              <div className="canvas-area-pointer" style={style} />
            </div>
          </div>
        </div>
      );

    let rgb = { ...color } as ColorProperty;
    if (type === PropertyType.LinearColor || type === PropertyType.Vector4)
      rgb = WidgetUtilities.colorToRgb(color, max);
        
    const hsv = WidgetUtilities.rgb2Hsv(rgb);
    rgb = WidgetUtilities.hsv2rgb({ h: hsv.h, s: 1, v: 1, a: hsv?.a });

    if (rgb?.A && type === PropertyType.Color)
      rgb.A /= 255;      

    const rectangleStyle: React.CSSProperties = { backgroundColor: `rgba(${rgb.R}, ${rgb.G}, ${rgb.B}, ${rgb?.A ?? 1})` };
    const rectanglePointerStyle: React.CSSProperties = { transform: `translate(${pos.rectangleX}px, ${pos.rectangleY}px)` };    

    return (
      <div className="rectangle"
           ref={this.rectangleRef}
           style={rectangleStyle}
           onPointerMove={this.onPointerHsvMove}
           onPointerDown={this.onPointerHsvDown}
           onPointerUp={this.onPointerHsvUp}>
        <div className="rectangle-area-pointer" style={rectanglePointerStyle} />
      </div>
    );
  }

  render() {
    const { label, type, widget, value, alpha, reset } = this.props;
    const { color, max } = this.state;

    const visible = this.getVisibleParts();

    let rgb = color as ColorProperty;
    if (type === PropertyType.LinearColor || type === PropertyType.Vector4)
      rgb = WidgetUtilities.colorToRgb(color, max);

    const hsv = WidgetUtilities.rgb2Hsv(rgb);
    let a = hsv?.a ?? 0;

    if (type === PropertyType.Color)
      a /= 255;

    return (
      <div className="color-picker-container">
        {!!this.props?.onPrecisionModal && <FontAwesomeIcon icon={['fas', 'expand']} className="expand-icon" onClick={this.props.onPrecisionModal} />}
        <div className="color-picker-wheel">
          <div>{label}</div>
          {!!visible[ColorPickerParts.Circle] && this.rendeCircleRectangle()}
          <div className='color-picker-controls'>
            {!!visible[ColorPickerParts.Wheel] && (
              <div className="slider-wheel-container">
                {widget === WidgetTypes.ColorPicker && (
                  <div className="slider-wheel-top">
                    <span className="title">Hue</span>
                    <span className="value">{(hsv.h * 360).toFixed(1)}°</span>
                  </div>
                )}
                <SliderWheel onWheelMove={this.onHsvWheelMove} />
              </div>
            )}
            {!!visible[ColorPickerParts.Saturation] && (
              <ColorPickerSlider value={hsv.s}
                                 label="Saturation"
                                 onChange={this.onValueSliderChange.bind(this, 's')} />
            )}
            {!!visible[ColorPickerParts.Value] && (
              <ColorPickerSlider value={hsv.v}
                                 label="Value"
                                 onChange={this.onValueSliderChange.bind(this, 'v')} />
            )}
            {!!alpha && !!visible[ColorPickerParts.Alpha] && (
              <ColorPickerSlider value={a}
                                 label="Alpha"
                                 onChange={this.onValueSliderChange.bind(this, 'a')} />
            )}
            
            <div className="color-picker-inputs">
              {!!visible[ColorPickerParts.Inputs] && (
                <div className='rgb-inputs'>
                  {this.renderRgbInput(SliderType.R)}
                  {this.renderRgbInput(SliderType.G)}
                  {this.renderRgbInput(SliderType.B)}
                </div>
              )}
              {widget === WidgetTypes.ColorPicker && !!value && <div className="color" style={this.getPreviewColorStyles()} />}
            </div>  
          </div>
        </div>
        {reset && <FontAwesomeIcon icon={['fas', 'undo']} onClick={() => this.props.onChange?.()} />}
      </div>
    );
  }
};

type ColorPickerSliderProps = {
  label: string;
  value: number;

  onChange: (value: number) => void;
}

type ColorPickerSliderState = {
  x: number;
}

const SliderWidth = 200;

class ColorPickerSlider extends React.Component<ColorPickerSliderProps, ColorPickerSliderState> {

  state: ColorPickerSliderState = {
    x: null,
  }

  monitoring = false;
  dragValue: number = null;
  dragOffset: number = null;

  ref = React.createRef<HTMLDivElement>();
  pointer = React.createRef<HTMLDivElement>();

  componentDidUpdate(prevProps: ColorPickerSliderProps, prevState: ColorPickerSliderState) {
    const { value } = this.props;

    if (!isNaN(value) && prevProps.value !== value)
      this.updateValues();
  }

  updateValues = () => {
    const { value } = this.props;
    const x = Math.min(SliderWidth - 4, Math.max(0, (SliderWidth - 4) * value));

    this.setState({ x });
  }

  onPointerDown = (e: React.PointerEvent<HTMLDivElement>) => {
    if (!this.ref.current)
      return;

    const pointer = this.pointer.current.getBoundingClientRect();

    this.dragOffset = 0;
    this.dragValue = this.props.value;

    if (pointer.left <= e.clientX && e.clientX <= pointer.right)
      this.dragOffset = e.clientX - (pointer.left + (pointer.width / 2));

    this.monitoring = true;
    this.ref.current.setPointerCapture(e.pointerId);
    this.onPointerMove(e);
  }

  onPointerMove = (e: React.PointerEvent<HTMLDivElement>) => {
    if (!this.monitoring)
      return;

    const slider = this.ref.current;
    const rect = slider.getBoundingClientRect();

    const x = e.clientX - this.dragOffset;
    const value = Math.max(0, Math.min((x - rect.left) / rect.width, 1));

    if (Math.abs(this.dragValue - value) > 0.001) {
      this.dragValue = value;

      this.setState({ x: Math.max(0, (x - rect.left) - 4) });
      this.props.onChange(value);
    }
  }

  onPointerLostCapture = (e: React.PointerEvent<HTMLDivElement>) => {
    if (!this.monitoring)
      return;

    this.dragOffset = 0;
    this.dragValue = null;
    this.monitoring = false;
    this.ref.current.releasePointerCapture(e.pointerId);
  }

  getPercentage = () => {
    const { x } = this.state;
    return Math.max(0, Math.round((100 * x) / (SliderWidth - 4)));
  }

  render() {
    const { label } = this.props;
    const { x } = this.state;

    return (
      <div className="color-picker-slider"
           ref={this.ref}
           style={{ width: SliderWidth }}
           onPointerMove={this.onPointerMove}
           onPointerDown={this.onPointerDown}
           onPointerUp={this.onPointerLostCapture}>
        <div className="color-picker-slider-label">{label}</div>
        <div className="color-picker-slider-pointer" style={{ transform: `translateX(${x}px)` }} ref={this.pointer} />
        <div className="color-picker-slider-bgc" />
        <div className="color-picker-slider-value">{this.getPercentage()}%</div>
      </div>
    );
  }
}
