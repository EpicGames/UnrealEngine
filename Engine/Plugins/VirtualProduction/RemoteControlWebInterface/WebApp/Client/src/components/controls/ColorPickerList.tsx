import React from 'react';
import { ColorPicker, ColorPickerValue } from './';
import { ColorProperty, IColorPickerList, ICustomStackProperty, PropertyType, PropertyValue } from 'src/shared';
import { DraggableWidget } from '../';
import _ from 'lodash';
import { WidgetUtilities } from 'src/utilities';
import { FontAwesomeIcon } from '@fortawesome/react-fontawesome';


type Props = {
  list: IColorPickerList;
  path: string;
  selection: string;
  isDragDisabled: boolean;
  dragging: string;

  onSelect?: (selection: string) => void;
  getPropertyValue?: (property: string) => PropertyValue;
  getPropertyLabel?: (widget: ICustomStackProperty) => string | JSX.Element;
  getAlpha?: (property: string) => boolean;
  onChange?: (widget: Partial<ICustomStackProperty>, value: PropertyValue) => void;
  onPrecisionModal?: (widget: ICustomStackProperty) => void;
}

type State = {
  selected: ICustomStackProperty;
}

export class ColorPickerList extends React.Component<Props, State> {

  state: State = {
    selected: null,
  }

  componentDidMount() {
    const { list } = this.props;
    const item = _.first(list.items);

    this.setState({ selected: item });
  }

  componentDidUpdate(prevProps: Props) {
    const { dragging, selection, list } = this.props;
    const { selected } = this.state;

    if (dragging !== prevProps.dragging && !!dragging) {
      const [, parent] = dragging.split('_');

      if (list.id === parent)
        this.setState({ selected: null });
    }

    if(list !== prevProps.list) {
      if (!selection) {
        const item = _.find(list.items, ['id', selected.id]) ?? _.first(list.items);
 
        return this.setState({ selected: item });
      }

      const [path, index] = selection.split('_');

      if (path === this.props.path)
        this.setState({ selected: list.items[index] });
    }
  }

  onItemSelect = (item: ICustomStackProperty, index: number, e: React.MouseEvent) => {
    const { path } = this.props;

    e.preventDefault();

    this.setState({ selected: item });
    this.props.onSelect(`${path}_${index}_${item.property}`);
  }

  getMaxColorValue = (type: PropertyType, color: ColorPickerValue) => {
    let max = 1;

    if (type === PropertyType.Vector4 || type === PropertyType.LinearColor) {
      max = 0;

      for (const key in color)
        if (color[key] > max)
          max = color[key];
    }

    return max;
  }

  getPreviewColorStyles = (color: ColorPickerValue, type: PropertyType, max: number): React.CSSProperties => {
    let rgb = color as ColorProperty;
    if (type === PropertyType.LinearColor || type === PropertyType.Vector4)
      rgb = WidgetUtilities.colorToRgb(color, max);

    return { background: `rgb(${rgb?.R}, ${rgb?.G}, ${rgb?.B})` };
  }

  render() {
    const { list, selection, path, isDragDisabled } = this.props;
    const { selected } = this.state;
    const value = this.props.getPropertyValue(selected?.property) as ColorPickerValue;

    return (
      <div className="color-pickers-list-container">
        <ul className="colors-list">
          {list?.items?.map((item, i) => {
            const draggableId = `${item.id}_${list.id}_${item.widget}`;
            const value = this.props.getPropertyValue(item?.property) as ColorPickerValue;
            const max = this.getMaxColorValue(item.propertyType, value);

            let className = 'list-item ';

            if (selected?.id === item.id)
              className += 'selected ';

            return (
              <li className={className} key={i}>
                <DraggableWidget draggableId={draggableId}
                                 key={item.id}
                                 selected={selection === `${path}_${i}_${item.property}`}
                                 index={i}
                                 onSelect={this.onItemSelect.bind(this, item, i)}
                                 isDragDisabled={isDragDisabled}>
                  <div>
                    {!!this.props.onPrecisionModal && 
                      <FontAwesomeIcon icon={['fas', 'expand']} className="expand-icon" onClick={() => this.props.onPrecisionModal(item)} />
                    }
                    {this.props.getPropertyLabel(item)}
                  </div>
                  <div className="color-preview" style={this.getPreviewColorStyles(value, item.propertyType, max)} />
                </DraggableWidget>
              </li>
            );
          })}
        </ul>
        <ColorPicker value={value} 
                     type={selected?.propertyType}
                     widget={selected?.widget}
                     alpha={this.props.getAlpha(selected?.property)}
                     onChange={this.props.onChange.bind(this, selected)}
                     onPrecisionModal={() => this.props.onPrecisionModal(selected)} />
      </div>
    );
  }
};
