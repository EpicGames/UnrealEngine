import React from 'react';
import { ValueInput } from '.';
import { FontAwesomeIcon } from '@fortawesome/react-fontawesome';


export enum DialMode {
  Range     = 'RANGE',
  Endless   = 'ENDLESS',
  Rotation  = 'ROTATION',
}

type Props = {
  min?: number;
  max?: number;
  startAngle?: number;
  endAngle?: number;
  value?: number;
  largeLines?: number;
  shortLines?: number;
  display?: 'VALUE' | 'PERCENT';
  mode?: DialMode;
  size?: number;
  label?: React.ReactNode;
  hideReset?: boolean;
  hidePrecision?: boolean;

  onChange?: (value?: number) => void;
  onPrecisionModal?: () => void;
};

export class Dial extends React.Component<Props> {
  static defaultProps: Props = {
    startAngle: 50,
    endAngle: 310,
    largeLines: 11,
    shortLines: 5,
    display: 'VALUE',
    mode: DialMode.Range,
    size: 202,
  };


  ref = React.createRef<HTMLDivElement>();
  monitoring: boolean = false;
  beautify = new Intl.NumberFormat();
  svgCircleDegreeLength: number = 477.5;

  onDown = (e: React.PointerEvent<HTMLDivElement>): void => {
    this.monitoring = true;
    this.ref.current.setPointerCapture(e.pointerId);
    this.onMove(e);
  }

  onUp = (e: React.PointerEvent<HTMLDivElement>): void => {
    this.monitoring = false;
    this.ref.current.releasePointerCapture(e.pointerId);
  }

  onMove = (e: React.PointerEvent<HTMLDivElement>) => {
    if (!this.monitoring) 
      return;

    const rect = this.ref.current.getBoundingClientRect();
    const xCenter = (rect.right - rect.left) / 2 + rect.left;
    const yCenter = (rect.bottom - rect.top) / 2 + rect.top;
    const radians = Math.atan2(e.clientX - xCenter, e.clientY - yCenter);
    
    let angle = -1 * radians * (180 / Math.PI);
    if (this.props.mode !== DialMode.Range)
      angle += 180;

    const { min, max, mode } = this.props;

    angle = this.normalize(angle);
    let value = this.getValueFromAngle(angle);

    if (mode === DialMode.Rotation) {
      value = this.normalize(value, -180, 180);
    } else {
      if (min !== undefined)
        value = Math.max(min, value);

      if (max !== undefined)
        value = Math.min(value, max);
    }

    this.props.onChange?.(value);
  }

  onValueInputChange = (value: number) => {
    const { mode } = this.props;

    if (mode === DialMode.Rotation)
      value = this.normalize(value, -180, 180);

    return this.props.onChange(value);
  }

  renderSmallLines = (linesList: JSX.Element[], angle: number, largeLineStep: number, tickAngle: number) => {
    const { shortLines, mode } = this.props;
    const rangeAngle = mode === DialMode.Range ? 180 : 0;
    const isActive = (mode !== DialMode.Range);

    for (let i = 1; i <= shortLines; i++) {
      const shortTickAngle = this.normalize(tickAngle + (largeLineStep / (shortLines + 1)) * i - 180);
      const active = angle >= this.normalize(shortTickAngle + rangeAngle) || isActive;
      const className = `dial-tick short ${active && 'active'}`;
      linesList.push(
        <div key={shortTickAngle} className={className} style={{ transform: `rotate(${shortTickAngle}deg)` }}/>
      );
    }
  }

  renderLines = (angle: number) => {
    let { startAngle, endAngle, largeLines, mode } = this.props;
    let endlessAngle = 0;
    const isActive = (mode !== DialMode.Range);

    if (isActive) {
      startAngle = 0;
      endAngle = 360;
      endlessAngle = 180;
    }

    angle = this.normalize(angle - endlessAngle);
    const largeLineStep = (endAngle - startAngle) / (largeLines - 1);
    if (largeLines < 1)
      return null;

    const linesList = [];
    for (let i = 0; i < largeLines; i++) {
      const tickAngle = this.normalize(i * largeLineStep + startAngle);
      const active = angle >= tickAngle || isActive;
      const className = `dial-tick large ${active && 'active'}`;
      linesList.push(
        <div key={tickAngle} 
              className={className}
              style={{ transform: `rotate(${tickAngle - 180 - endlessAngle}deg)` }} />
      );

      if (i + 1 < largeLines)
        this.renderSmallLines(linesList, angle, largeLineStep, tickAngle);
    }

    return linesList;
  }

  normalize = (angle: number, begin: number = 0, end: number = 360): number => {
    const range = end - begin;
    while (angle < begin) 
      angle += range;

    while (angle > end)
      angle -= range;

    return angle;
  }

  getValue = () => {
    const { min, max } = this.props;
    let { value = 0 } = this.props;

    if (!isNaN(min))
      value = Math.max(min, value);

    if (!isNaN(max))
      value = Math.min(max, value);
      
    return value;
  }

  getValueFromAngle = (angle: number) => {
    const { min, startAngle, endAngle, mode } = this.props;
    const range = this.getRange();

    switch (mode) {
      case DialMode.Range:
        angle = Math.max(startAngle, Math.min(angle, endAngle));
        const ratio = (angle - startAngle) / (endAngle - startAngle);
        return min + ratio * range;

      case DialMode.Endless:
      case DialMode.Rotation:
        let value = this.getValue();
        const prevAngle = this.normalize(this.getAngle(value) - 180);
        const phi = this.normalize(angle - prevAngle);
        const delta = phi > 180 ? phi - 360 : phi;
        const step = range >= 360 ? 1 : range / 1080;
        value += delta * step;
        if (mode === DialMode.Rotation)
          value = this.normalize(value);
        return value;
    }
  }

  getRange = () => {
    const { min, max } = this.props;
    if (min === undefined || max === undefined)
      return 1000;

    return Math.max(max - min, 0.01);
  }
  
  getAngle = (value: number) => {
    const { mode, min = 0, startAngle, endAngle } = this.props;
    const range = this.getRange();

    let angle = 0;
    switch (mode) {
      case DialMode.Range:
        const ratio = (value - min) / range;
        angle = startAngle + ratio * (endAngle - startAngle);
        break;

      case DialMode.Endless:
      case DialMode.Rotation:
        const step = range >= 360 ? 1 : range / 1080;
        angle = 180 + (value / step) % 360;
        break;
    }

    return this.normalize(angle);
  }

  getText = (value: number): string => {
    if (isNaN(value))
      return '';

    const { min, max, display } = this.props;
    const range = Math.max(0.1, max - min);
    const decimals = Math.max(0, 4 - range.toFixed().length);
    const ratio = (value - min) / range;
    if (display === 'PERCENT')
      return (ratio * 100).toFixed(2);

    return this.beautify.format(+value?.toFixed(decimals));
  }

  getCirclePaintOver = (angle: number) => {
    const { startAngle, mode } = this.props;
    
    const calcLength = (a:number) => {
      const angleInProc = a / 3.6;
      return this.svgCircleDegreeLength / 100 * angleInProc;
    };

    switch (mode) {
      case DialMode.Endless:
      case DialMode.Rotation:
        const angleCalc = angle < 180 ? angle + 180 : angle - 180;
        return this.svgCircleDegreeLength - calcLength(angleCalc);

      case DialMode.Range: 
        return this.svgCircleDegreeLength + calcLength(startAngle) - calcLength(angle);
    }
  }

  getPaintOverRotate = () => {
    const { startAngle, mode } = this.props;
    return mode === DialMode.Range ? 90 + startAngle : -90;
  }

  render() {
    const { value = 0, size, min, max, label, mode } = this.props;
    const angle = this.getAngle(value);
    const wrapperStyle: React.CSSProperties = { width: size, height: size };
    const dialInputStyle: React.CSSProperties =  mode === DialMode.Range ? { position: 'absolute', top: '82%' } : {};

    return ( 
      <div className="dial-widget-wrapper">
        {!this.props.hidePrecision &&
          <FontAwesomeIcon icon={['fas', 'expand']} className="expand-icon" onClick={this.props.onPrecisionModal} />
        }
        {label &&
          <div className="dial-label">{label}</div>
        }
        <div className="dial-border-first">
          <div className="dial-border-second">
            <div className="dial-wrapper"
                  ref={this.ref}
                  style={!isNaN(size) ? wrapperStyle : {}}
                  onLostPointerCapture={this.onUp}
                  onPointerDown={this.onDown}
                  onPointerUp={this.onUp}
                  onPointerMove={this.onMove}
                  tabIndex={-1}>
              <div className="dial-circle">
                <Circle angle={angle}
                        size={size}
                        mode={mode}
                        rotate={this.getPaintOverRotate()}
                        circlePaintOver={this.getCirclePaintOver(angle)}
                        svgCircleDegreeLength={this.svgCircleDegreeLength} />
                <div className="dial-ticks">
                  {this.renderLines(angle)}
                </div>
                <div className="dial-center-wrapper">
                  <div className="dial-center">
                    {label &&
                      <div className="dial-label">{label}</div>
                    }
                    <div className="dial-value" style={dialInputStyle} onPointerDown={e => e.stopPropagation()}>
                      <ValueInput min={min} max={max} value={value} onChange={this.onValueInputChange} />
                    </div>
                  </div>
                </div>
                <div className="dial-line" style={{ transform: `rotate(${angle}deg)` }} />
              </div>
            </div>
          </div>
        </div>
        {!this.props.hideReset &&
          <FontAwesomeIcon icon={['fas', 'undo']} onClick={() => this.props.onChange?.()} />
        }
      </div>     
    );
  }
}


type CircleProps = {
  angle: number;
  size: number;
  mode: DialMode;
  rotate: number;
  circlePaintOver:  number;
  svgCircleDegreeLength: number;
};

class Circle extends React.Component<CircleProps> {
  render() {
    const { angle, size, rotate, circlePaintOver, svgCircleDegreeLength, mode } = this.props;

    const cxcy = size / 2 - 1;

    let circleRotateAngle = 0;
    let strokeDashoffset = circlePaintOver;
    if (mode !== DialMode.Range) {
      const angleCalc = angle < 180 ? angle + 180 : angle - 180;
      circleRotateAngle = angleCalc - 90;
      
      const rotatePart = svgCircleDegreeLength / 4 * 3;
      strokeDashoffset = rotatePart;
    }

    let className = 'svg-circle';
    if (mode === DialMode.Range)
      className += ' range';

    return (
      <div className="svg-circle">
        <svg xmlns="http://www.w3.org/2000/svg" width="100%" height="100%">
          <defs>
            <linearGradient id="shape-gradient" x1="0" x2="0" y1="1" y2="0">
              <stop offset="0%" stopColor="rgba(0,103,180,1)" />
              <stop offset="50%" stopColor="rgba(0,103,180,0)" />
            </linearGradient>
          </defs>
          <circle transform={`rotate(${rotate + circleRotateAngle}, ${cxcy}, ${cxcy})`}
                  cx={cxcy}
                  cy={cxcy}
                  r="76"
                  fill="transparent"
                  strokeWidth="45"
                  strokeDasharray={svgCircleDegreeLength}
                  strokeDashoffset={strokeDashoffset}
                  className={className} />
        </svg>
      </div>
    );
  }
}