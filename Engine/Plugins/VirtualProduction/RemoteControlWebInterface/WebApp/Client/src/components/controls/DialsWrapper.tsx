import React from 'react';
import { Dial, DialMode } from '.';
import { WidgetUtilities } from 'src/utilities';
import { JoystickValue, PropertyType } from 'src/shared';
import { FontAwesomeIcon } from '@fortawesome/react-fontawesome';


type Props = {
  min?: number;
  max?: number;
  display?: 'VALUE' | 'PERCENT';
  mode?: DialMode;
  size?: number;
  properties?: string[];
  value?: number | JoystickValue;
  type?: PropertyType;
  hidePrecision?: boolean;
  label?: React.ReactNode;
  proportionally?: boolean;
  property?: string;

  onChange?: (res:  number | JoystickValue) => void;
  onProportionallyToggle?: (property: string, value: string) => void;
};

export class DialsWrapper extends React.Component<Props> {

  getProperties = (): string[] => {
    const { type, properties } = this.props;
    let keys = WidgetUtilities.getPropertyKeys(type);

    if (properties?.length)
      keys = keys.filter(key => properties.includes(key));

    return keys;
  }

  onChange = (res: number, key?: string) => {
    let { value, proportionally, min, max } = this.props;
    if (value === undefined) {
      const dials = this.getProperties();
      value = dials.length ? {} : 0;
    }

    let prev = value[key];
    if (prev === 0 || prev === undefined)
      prev = 1;

    const ratio = Math.max(0.001, res) / prev;

    if (proportionally && !isNaN(ratio)) {
      for (const key of Object.keys(value)) {
        let val = value[key] * ratio;
        if (!isNaN(min) && !isNaN(max))
          val = Math.min(max, Math.max(min, val));

        value[key] = val;
      }
    }
    else {
      if (key)
        value[key] = res;
      else
        value = res;
    }

    this.props.onChange?.(value);
  }

  renderProportionallyIcon = () => {
    const { proportionally, property } = this.props;

    if (proportionally === undefined)
      return null;

    return <FontAwesomeIcon icon={['fas', (proportionally) ? 'lock' : 'lock-open']}
                            className='proportional icon'
                            onClick={() => this.props.onProportionallyToggle(property, proportionally ? '0' : '1')} />;
  }

  render() {
    const { min, max, mode, value, display, hidePrecision, label } = this.props;
    const { size } = this.props;
    const dials = this.getProperties();

    let dialSize = 203;
    if (size)
      dialSize = Math.min(size / dials.length, size - 70) - 10;

    let startAngle, endAngle;

    const props = {
      mode,
      min,
      max,
      display,
      size: dialSize,
      startAngle,
      endAngle,
      hideReset: true,
      hidePrecision,
    };

    return (
      <div className="dial-wrapper-container">
        {label}
        {this.renderProportionallyIcon()}
        <div className="dial-wrapper-block">
          {!!dials.length &&
            dials.map(key => {
              return (
                <Dial {...props}
                      key={key}
                      label={key}
                      value={value?.[key] ?? 0}
                      onChange={value => this.onChange(value, key)} />
              );
            })
          }

          {!dials.length &&
            <Dial {...props}
                  value={typeof value === 'number' ? value : 0}
                  onChange={this.onChange} />
          }
        </div>
      </div>
    );
  }
}
