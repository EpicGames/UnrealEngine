import React from 'react';
import { JoystickValue } from 'src/shared';
import { FontAwesomeIcon } from '@fortawesome/react-fontawesome';


type Props = {
  value?: JoystickValue;
  label?: string;
  step?: number;
  interval?: number;
  min?: number;
  max?: number;

  onChange?: (res: JoystickValue) => void;
};

type State = {
  x: number;
  y: number;
};

export class Joystick extends React.Component<Props, State> {

  static defaultProps: Props = {
    interval: 100,
  }

  state: State = {
    x: 0,
    y: 0,
  };

  ref = React.createRef<HTMLDivElement>();
  monitoring = false;
  step: JoystickValue = {};
  value: JoystickValue = null;
  interval = null;

  onPointerDown = (e: React.PointerEvent<HTMLDivElement>) => {
    const { interval, value } = this.props;

    this.monitoring = true;
    this.ref.current.setPointerCapture(e.pointerId);
    this.value = value;

    this.onPointerMove(e);
    this.makeMove();
    this.interval = setInterval(this.makeMove, interval);
  }

  onPointerMove = (e: React.PointerEvent<HTMLDivElement>) => {
    if (!this.monitoring)
      return;

    const { label, step } = this.props;
    const rect = this.ref.current.getBoundingClientRect();

    const xCenter = (rect.right - rect.left) / 2 + rect.left;
    const yCenter = (rect.bottom - rect.top) / 2 + rect.top;

    let x = e.clientX - xCenter;
    let y = e.clientY - yCenter;

    const radius = Math.sqrt(x * x + y * y);
    const maxRadius = this.ref.current.clientHeight / 2;

    const radians = Math.atan2(x, y);
    const angle = this.normalize(-1 * radians * (180 / Math.PI) - 90) / 360;

    const r = Math.min(radius / maxRadius, 0.88);
    const radian = (angle - 0.25) * Math.PI * 2;

    x = Math.sin(radian) * r * 90;
    y = Math.cos(radian) * r * 90;

    let joystickValue = {} as JoystickValue;    

    switch (label) {
      case 'X':
        x = 0;
        joystickValue.X = (y * step) / maxRadius;
        break;

      case 'Y':
        y = 0;
        joystickValue.Y = (x * step) / maxRadius;
        break;

      case 'Z':
        x = 0;
        joystickValue.Z = (y * step) / maxRadius;
        break;

      case 'XY':
        joystickValue.X = (y * step) / maxRadius;
        joystickValue.Y = (x * step) / maxRadius;
        break;
    }

    this.step = joystickValue;
    this.setState({ x, y });
  }

  onPointerLost = (e: React.PointerEvent<HTMLDivElement>) => {
    if (!this.monitoring)
      return;

    this.ref.current.releasePointerCapture(e.pointerId);
    this.monitoring = false;
    this.value = null;
    
    this.setState({ x: 0, y: 0 });
    clearInterval(this.interval);
    this.interval = null;
  }

  makeMove = () => {
    const { min, max } = this.props;

    if (!this.value)
      return;

    for (const key in this.step) {
      let v = this.value[key] + this.step[key];
      if (max !== undefined)
        v = Math.min(v, max);

      if (min !== undefined)
        v = Math.max(min, v);

      this.value[key] = v;
    }
    
    this.props.onChange?.(this.value);
  }

  normalize = (angle: number): number => {
    while (angle < 0)
      angle += 360;

    while (angle > 360)
      angle -= 360;

    return angle;
  }

  render() {
    const { x, y } = this.state;
    const { label } = this.props;

    let className = 'circle ';
    if (!this.monitoring)
      className += 'animate';

    return (
      <div className="joystick-wrapper">
        <div ref={this.ref}
              className="joystick"
              onPointerDown={this.onPointerDown}
              onPointerMoveCapture={this.onPointerMove}
              onLostPointerCaptureCapture={this.onPointerLost}>
          <div className="arrows">
            {['Z', 'X', 'XY'].includes(label) && <FontAwesomeIcon icon={['fas', 'caret-up']} />}
            {['Y', 'XY'].includes(label) && <FontAwesomeIcon icon={['fas', 'caret-left']} />}
            {['Y', 'XY'].includes(label) && <FontAwesomeIcon icon={['fas', 'caret-right']} />}
            {['Z', 'X', 'XY'].includes(label) && <FontAwesomeIcon icon={['fas', 'caret-down']} />}
          </div>
          <div className="circle-wrapper">
            <div className={className} style={{ transform: `translate(${x}px, ${-y}px)` }}>
              <div className="label">{label}</div>
            </div>
          </div>
        </div>
      </div>
    );
  }
}