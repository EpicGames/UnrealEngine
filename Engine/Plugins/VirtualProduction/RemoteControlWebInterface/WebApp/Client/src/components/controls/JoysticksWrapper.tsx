import { FontAwesomeIcon } from '@fortawesome/react-fontawesome';
import React from 'react';
import { JoystickValue, PropertyType } from 'src/shared';
import { Slider, ValueInput, Joystick } from '.';


type Props = {
  keys?: string[];
  value?: JoystickValue;
  min?: number;
  max?: number;
  type?: PropertyType;
  stepMin?: number;
  stepMax?: number;
  showStepSlider?: boolean;
  step?: number;
  hasLimits?: boolean;
  showReset?: boolean;
  label?: React.ReactNode;

  onChange?: (value?: JoystickValue) => void;
};

type State = {
  speed: number;
}

export class JoysticksWrapper extends React.Component<Props, State> {

  state: State = {
    speed: 5,
  }

  onSpeedChange = (speed: number) => {
    this.setState({ speed });
  }

  getJoystick = (pos: 'left' | 'right') => {
    const { keys = [] } = this.props;

    const x = keys[0];
    const y = keys[1];
    const z = keys[2];

    switch (pos) {
      case 'left':
        return (z ? x + y : x);

      case 'right':
        return (z ? z : y);
    }
  }

  renderInput = (axis: string, value: JoystickValue, min: number, max: number) => {
    return (
      <div key={axis} className="control">
        <span className="label">{axis}:</span>
        <ValueInput min={min}
                    max={max}
                    value={value?.[axis]}
                    onChange={v => this.props.onChange?.({...value, [axis]: v })} />
      </div>
    );
  }

  render() {
    const { value, min, max, label = '', keys = [], showReset = true } = this.props;
    const { speed } = this.state;
    
    const leftJoystick = this.getJoystick('left');
    const rightJoystick = this.getJoystick('right');

    return (
      <div className="joystick-outer-wrapper">
        {showReset &&
          <FontAwesomeIcon className="reset-btn" icon={['fas', 'undo']} onClick={() => this.props.onChange?.()} />
        }
        <div className="joystick-wrapper-block">
          <Joystick label={leftJoystick}
                    value={value}
                    step={speed}
                    min={min}
                    max={max}
                    onChange={this.props.onChange} />
        </div>

        <div className="controls-block">
          <div className="label">{label}</div>
          <div className="inputs">
            {keys.map(key => this.renderInput(key, value, min, max))}
          </div>

          <div className="slider">
            <Slider value={speed}
                    min={0.01}
                    max={100}
                    onChange={this.onSpeedChange}
                    showLabel={false} />
            <div className="values">
              <div className="field">Slow</div>
              <ValueInput value={speed} min={0.01} onChange={this.onSpeedChange} />
              <div className="field">Fast</div>
            </div>
            <div className="centered">Speed</div>
          </div>
        </div>

        <div className="joystick-wrapper-block">
          <Joystick label={rightJoystick}
                    value={value}
                    step={speed}
                    min={min}
                    max={max}
                    onChange={this.props.onChange} />
        </div>
      </div>
    );
  }
}