import React from 'react';
import { FontAwesomeIcon } from '@fortawesome/react-fontawesome';


type Props = {
  label?: string;
  fallback?: string;
};

type State = {
  hasError: boolean;
};

export class SafeImage extends React.Component<React.ImgHTMLAttributes<HTMLImageElement> & Props, State> {
  state: State = {
    hasError: false,
  };

  render() {
    const { fallback, alt, label, style } = this.props;

    const props = {...this.props};
    delete props.fallback;
    delete props.label;

    if (!this.state.hasError)
      return <img {...props} alt={alt ?? label} onError={() => this.setState({ hasError: true })} />;

    if (fallback)
      return <img {...props} src={fallback} alt={alt ?? label} />;

    return (
      <div className="safe-image-wrapper" style={style}>
        <FontAwesomeIcon icon={['fas', 'image']} />
        <div className="image-label">Can't load image</div>
        <div className="image-label">{label}</div>
      </div>
    );
  }

};