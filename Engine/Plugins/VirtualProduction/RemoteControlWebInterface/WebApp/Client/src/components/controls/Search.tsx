import React from 'react';
import { FontAwesomeIcon } from '@fortawesome/react-fontawesome';


type Props = {
  placeholder?: string;

  onSearch: (value: string) => void;
};

type State = {
  filter: string,
};

export class Search extends React.Component<Props, State> {
  static defaultProps: Partial<Props> = {
    placeholder: 'Search',
  };

  state: State = {
    filter: '',
  };

  componentDidUpdate(prevProps) {
    const { placeholder, onSearch } = this.props;
    
    if(placeholder !== prevProps.placeholder) {
      this.setState({ filter: '' });
      onSearch('');
    }
  }

  componentWillUnmount() {
    this.props.onSearch?.('');
  }

  onSearch = (filter: string) => {
    this.props.onSearch(filter.toLowerCase());

    this.setState({ filter });
  }

  render() {    
    const { filter } = this.state;
    const { placeholder } = this.props;

    return (
      <div className="search-wrapper">
        <FontAwesomeIcon icon={['fas', 'search']} />
        <input className="list-search" value={filter} placeholder={placeholder} onChange={e => this.onSearch(e.target.value)}/>
        {filter && 
          <FontAwesomeIcon icon={['fas', 'times-circle']} onClick={() => this.onSearch('')} />
        }
      </div>
    );
  }
}