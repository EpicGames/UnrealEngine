import React from 'react';
import { ValueInput } from './ValueInput';


type Props = {
  min?: number;
  max?: number;
  size?: number;
  value?: number;
  showGrid?: boolean;
  showLimits?: boolean;
  showLabel?: boolean;
  precision?: number;
  touchMode?: boolean;
  style?: React.CSSProperties;
  
  onChange?: (value: number) => void;
};

export class Slider extends React.Component<Props> {

  static defaultProps: Props = {
    style: {},
  };

  ref = React.createRef<HTMLDivElement>();
  circleRef = React.createRef<HTMLDivElement>();
  monitoring: boolean = false;
  dragValue: number = null;
  dragOffset: number = null;

  onDown = (e: React.PointerEvent<HTMLDivElement>) => {
    const { touchMode } = this.props;
    const target = e.target as HTMLDivElement;

    if (touchMode && !target.classList.contains('circle'))
      return;

    this.monitoring = true;
    this.ref.current.setPointerCapture(e.pointerId);

    const circle = this.circleRef.current.getBoundingClientRect();

    this.dragOffset = 0;
    this.dragValue = this.props.value;
    if (circle.left <= e.clientX && e.clientX <= circle.right)
      this.dragOffset = e.clientX - (circle.left + (circle.width / 2));
    
    this.onMove(e);
  }

  onUp = (e: React.PointerEvent<HTMLDivElement>) => {
    this.monitoring = false;
    this.dragOffset = 0;
    this.dragValue = null;
    this.ref.current.releasePointerCapture(e.pointerId);
  }
  
  onMove = (e: React.PointerEvent<HTMLDivElement>) => {
    if (!this.monitoring || !this.ref.current)
      return;

    const { min, max } = this.props;
    const slider = this.ref.current;

    const rect = slider.getBoundingClientRect();
    const x = e.clientX - this.dragOffset;
    const ratio = Math.max(0, Math.min((x - rect.left) / rect.width, 1));

    const value = ratio * (max - min) + min;
    if (this.dragValue !== value) {
      this.dragValue = value;
      this.props.onChange?.(value);
    }
  }

  renderGridLines = (percent: number) => {
    const res = [];
    for (let i = 0; i < 11; i++)
      res.push(<div key={i} className={`grid-lines ${percent > i * 9.9999 ? 'active' : ''}`} />);

    return <div className="grid-lines-wrapper">{res}</div>;
  }

  onKeyDown = (e: React.KeyboardEvent) => {
    const { value, min, max } = this.props;
    const step = (max - min) / 20;

    switch (e.nativeEvent.code) {
      case 'KeyA':
      case 'KeyS':
        this.props.onChange?.(value - step);
        break;

      case 'KeyW':
      case 'KeyD':
        this.props.onChange?.(value + step);
        break;
    };
  }

  render()  {
    const { min, max, showGrid, showLimits, showLabel, precision, touchMode, style } = this.props;
    const value = this.dragValue ?? this.props.value;

    let percent = 0;
    let className = 'slider ';

    if (touchMode)
      className += 'touch-mode';

    if (!isNaN(value)) {
      percent = (value - min) / (max - min) * 100;
      percent = Math.max(0, Math.min(percent, 100));
    }  

    return (
      <div className="slider-wrapper">
        {showLabel !== false &&
          <ValueInput min={min}
                      max={max}
                      precision={precision}
                      value={value}
                      onChange={this.props.onChange} />
        }

        <div className="slider-inner" tabIndex={0} onKeyDown={this.onKeyDown}>

          {showLimits && 
            <div className="label-wrapper">
              <span className="label">{min}</span>
              <span className="label">{max}</span>
            </div>
          }
          
          <div className="slider-clickable"
                ref={this.ref}
                onLostPointerCapture={this.onUp}
                onPointerDown={this.onDown}
                onPointerUp={this.onUp}
                onPointerMove={this.onMove}>
            <div className="slider-block">
              <div className={className} style={style} />
              <div className="range-fill" style={{ width: `${percent}%` }} />
              <div className="circle-wrapper" style={{ transform: `translateX(${percent}%)` }}>
                <div className="circle" 
                     ref={this.circleRef} />
              </div>
            </div>

            {showGrid && 
              this.renderGridLines(percent)
            }
          </div>

        </div>

      </div>
    );
  }
} 