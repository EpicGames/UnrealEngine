import React from 'react';


type SliderWheelProps = {
  vertical?: boolean;
  size?: number,
  className?: string;
  style?: React.CSSProperties;

  onWheelMove?: (value: number, offset?: number) => void;
  onWheelStart?: () => void;
}

type SliderWheelState = {
  offset: number;
}

export class SliderWheel extends React.Component<SliderWheelProps, SliderWheelState> {

  static defaultProps: SliderWheelProps = {
    style: {},
  };

  state: SliderWheelState = {
    offset: 0,
  };

  ref = React.createRef<HTMLDivElement>();
  monitoring: boolean = false;
  last: number = -1;
  sum: number = 0;

  onPointerDown = (e: React.PointerEvent<HTMLDivElement>) => {
    if (!this.ref.current)
      return;

    const { vertical } = this.props;  

    this.monitoring = true;
    this.last = e.clientX;

    if (vertical)
      this.last = e.clientY;

    this.ref.current.setPointerCapture(e.pointerId);
    this.props.onWheelStart?.();
  }

  onPointerMove = (e: React.PointerEvent<HTMLDivElement>) => {
    if (!this.monitoring)
      return;

    const { vertical } = this.props;
    let delta = e.clientX - this.last;

    if (vertical)
      delta = this.last - e.clientY;

    if (Math.abs(delta) < 2)
      return; 

    this.last = e.clientX;
    if(vertical)
      this.last = e.clientY;

    let { offset } = this.state;

    this.sum += delta;
    const rect = this.ref.current.getBoundingClientRect();

    offset += delta;
    offset %= rect.width / 2;

    this.props.onWheelMove?.(Math.sign(delta), this.sum / (rect.width / 2));
    this.setState({ offset });
  }

  onPointerUp = (e: React.PointerEvent<HTMLDivElement>) => {
    if (!this.monitoring)
      return;

    this.monitoring = false;
    this.last = 0;
    this.sum = 0;
    this.ref.current.releasePointerCapture(e.pointerId);
  }

  renderCircles = () => {
    let { size } = this.props;
    const circles = [];

    size = size ?? 80;

    for (let i = 0; i < size; i++)
      circles.push(<div key={i} className="slider-circle" />);

    return circles;
  }

  render() {
    const { offset } = this.state;
    let { vertical, className = '', } = this.props;
    const style: React.CSSProperties = { transform: `translateX(${offset}px)` };

    if (vertical)
      style.transform = `translateY(${offset}px)`;

    className += ' color-picker-slider-wheel ';
    if (vertical)
      className += 'vertical';

    return (
      <div className={className}
           onPointerMove={this.onPointerMove}
           onPointerDown={this.onPointerDown}
           onPointerUp={this.onPointerUp}
           ref={this.ref}
           style={this.props.style}>
        <div className="circles-list" style={style}>
          {this.renderCircles()}
        </div>
      </div>
    );
  }
}


