import React from 'react';
import { FontAwesomeIcon } from '@fortawesome/react-fontawesome';
import { ICustomStackProperty, JoystickValue, PropertyValue } from 'src/shared';
import { WidgetUtilities } from 'src/utilities';
import { Slider, ValueInput } from '.';


type Props = {
  widget?: ICustomStackProperty;
  label?: string;
  min: number;
  max: number;
  value: JoystickValue;

  onChange: (value?: PropertyValue) => void;
};

export class SlidersWrapper extends React.Component<Props> {

  onSliderChange = (axis: string, sliderValue: number) => {
    const { value = {}, onChange } = this.props;

    if (axis) {
      value[axis] = sliderValue;
      onChange(value);
    }
  }

  render() {
    const { min, max, value, widget } = this.props;
    const propertyType = widget?.propertyType;
    const keys = WidgetUtilities.getPropertyKeys(propertyType);
    const precision = WidgetUtilities.getPropertyPrecision(propertyType);

    return (
      <div className="sliders-component-wrapper">
        {keys.map(property =>
          <div key={property} className="slider-row">
            <div className="title">{property}</div>
            <ValueInput min={min}
                        max={max}
                        precision={precision}
                        value={value?.[property]}
                        onChange={value => this.onSliderChange(property, value)} />
            {min !== undefined && max !== undefined &&
              <>
                <div className="limits">{min?.toFixed(1)}</div>
                <Slider value={value?.[property]}
                        min={min}
                        max={max}
                        precision={precision}
                        showLabel={false}
                        onChange={value => this.onSliderChange(property, value)} />
                <div className="limits">{max?.toFixed(1)}</div>
              </>
  }
            <FontAwesomeIcon icon={['fas', 'undo']} onClick={() => this.props.onChange?.()} />
          </div>
        )}
      </div>
    );
  }
};