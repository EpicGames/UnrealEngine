import React from 'react';
import _ from 'lodash';
import { IconProp } from '@fortawesome/fontawesome-svg-core';
import { FontAwesomeIcon } from '@fortawesome/react-fontawesome';


type Props = {
  children: React.ReactElement[];

  leftIcon?: React.ReactElement[] | React.ReactElement;
  rightIcon?: React.ReactElement[] | React.ReactElement;

  defaultActiveKey?: string;
  onlyHeader?: boolean;

  onTabChange?: (active?: string) => void;
}

type State = {
  active: string;
}

export class Tabs extends React.Component<Props, State> {

  state: State = {
    active: null,
  }

  selectorRef = React.createRef<HTMLDivElement>();
  containerRef = React.createRef<HTMLDivElement>();

  componentDidMount() {
    this.updateActive();

    window.addEventListener('resize', this.updateSelector);
  }

  componentWillUnmount() {
    window.removeEventListener('resize', this.updateSelector);
  }

  componentDidUpdate(prevProps: Readonly<Props>, prevState: Readonly<State>): void {
    const { active } = this.state;
    const { defaultActiveKey, children } = this.props;

    if (children.length !== prevProps.children.length)
      this.updateSelector();

    if (active !== prevState.active)
      this.updateSelector();

    if (defaultActiveKey !== prevProps.defaultActiveKey)
      this.updateActive();
  }

  getSelectorPosition = () => {
    const elements = this.containerRef.current.getElementsByClassName('tab');
    if (!elements.length)
      return;

    let left = 5;
    let width = null;
    
    for(let i = 0; i < elements.length; i++) {
      const el = elements[i] as HTMLDivElement;

      if (el.classList.contains('active')) {
        width = ((el.offsetLeft - left) * 2) + el.clientWidth;
        break;
      }

      left = ((el.offsetLeft * 2) + el.clientWidth) - left;
    }

    return [left, width];
  }

  onTabChange = (active: string) => {
    this.setState({ active });
    this.props.onTabChange?.(active);
  }

  updateActive = () => {
    const { children, defaultActiveKey } = this.props;
    const child = _.first(children) as any;

    if (!child)
      return;

    const id = child.props.id;
    this.setState({ active: defaultActiveKey ?? id });
  }

  updateSelector = () => {
    const selector = this.selectorRef.current;
    const [left, width] = this.getSelectorPosition();

    selector.style.left = `${left}px`;
    selector.style.width = `${width}px`;
  }

  renderTabHeader = (child, index: number) => {
    if (!child)
      return null;

    const { active } = this.state;
    const { id, tab, icon, view } = child.props;

    const isActive = active === id || !active && index === 0;
    let className = 'tab ';

    if (isActive)
      className += 'active ';

    return (
      <div key={id}
           className={className}
           onClick={() => this.onTabChange(id)}
           title={tab}>
        {!!icon && <FontAwesomeIcon className="icon" icon={icon} />}
        {tab}
        {!!view && view({ ...child.props, active: isActive })}
      </div>
    );
  }

  renderTabContent = () => {
    const { active } = this.state;
    const { children } = this.props;

    const child = _.find(children, c => c?.props?.id === active);

    if (!child)
      return null;

    return child;
  }

  render() {
    let { children, leftIcon, rightIcon, onlyHeader } = this.props;
    children = Array.isArray(children) ? children : [children];

    return (
      <div ref={this.containerRef} className="tabs-container">
        <div className="tabs-header-elements">
          {!!leftIcon && leftIcon}
          <div className="tabs-elements">
            <div className="tabs-header">
              {children.map?.(this.renderTabHeader)}
            </div>
            <div className="selector" ref={this.selectorRef} />
          </div>
          {!!rightIcon && rightIcon}
        </div>
        {!onlyHeader && <div className="tabs-content">{this.renderTabContent()}</div>}
      </div>
    );
  }
};

type TabPaneProps = {
  tab: string;
  id: string;
  icon?: IconProp;

  view?: ({ id, tab, icon, active }) => React.ReactElement;
}

export class TabPane extends React.Component<TabPaneProps> {
  render() {
    return this.props.children;
  }
}