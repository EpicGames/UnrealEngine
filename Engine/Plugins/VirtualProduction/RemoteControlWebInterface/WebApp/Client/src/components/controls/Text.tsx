import React from 'react';

type Props = {
  value?: string;
  onChange?: (value: string) => void;
};

export class Text extends React.Component<Props> {

  onChange = (e: React.ChangeEvent<HTMLInputElement>) => {
    this.props.onChange?.(e.target.value);
  }

  render() {
    const { value } = this.props;

    return (
      <div className="widget-text-wrapper">
        <input className="widget-text" type="textbox" value={value ?? ''} onChange={this.onChange} />
      </div>
    );
  }
}
