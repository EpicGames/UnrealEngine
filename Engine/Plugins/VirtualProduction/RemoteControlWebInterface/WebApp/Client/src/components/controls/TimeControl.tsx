import { FontAwesomeIcon } from '@fortawesome/react-fontawesome';
import React from 'react';


type Props = {
  label?: string;
  value?: number;
  disabled?: boolean;

  onChange?: (value: number) => void;
}

type State = {
  value?: number;
}

enum TimeKey {
  Hours   = 'HOURS',
  Minutes = 'MINUTES',
}

export class TimeControl extends React.Component<Props, State> {

  static defaultProps: Props = {
    label: '',
    value: 23.5,
  };

  state: State = {
    value: null,
  }

  hoursInputRef = React.createRef<HTMLInputElement>();
  minutesInputRef = React.createRef<HTMLInputElement>();

  componentDidMount() {
    const { value } = this.props;
    this.setState({ value });
  }

  componentDidUpdate(prevProps: Props): void {
    // const { value } = this.props;
    
    // if (value !== prevProps.value) {
    //   this.setState({ value });
    // }
  }

  getTwoDigits = (num: number) => {
    return num.toString().padStart(2, '0');
  }

  getInputValues = () => {
    const { value } = this.state;

    if (value === null)
      return ':';

    const hours = Math.floor(value);
    const minutes = Math.round((value * 60) % 60);

    return [this.getTwoDigits(hours), this.getTwoDigits(minutes)];
  }

  normalizeValue = (value: number) => {
    if (value >= 13)
      value -= 12;

    if (value < 1)
      value += 12;

    return +value.toFixed(2);
  }

  onTimeChange = (dir: number) => {
    let { value } = this.state;

    value += dir * 0.25;
    value = this.normalizeValue(value);

    this.props.onChange?.(value);
    this.setState({ value });
  }

  onInputClick = (ref: React.RefObject<HTMLInputElement>) => {
    if (!ref?.current)
      return;

    ref.current.select();
  }

  onInputChange = (v: string, key: TimeKey) => {
    let { value } = this.state;
    const [h] = value.toString().split('.');
    const m = (value - +h).toFixed(2);

    let diff = 0;

    switch (key) {
      case TimeKey.Hours:
        diff = +v - +h;
        break;

      case TimeKey.Minutes:
        if (v.length > 2)
          v = v.slice(1, 3);

        const d = Math.min(59, +v);
        diff = +(d / 60).toFixed(2) - +m;
        break;
    }

    value = +(value + diff).toFixed(2);
    if (value >= 13 || value < 1)
      return;

    this.props.onChange?.(value);
    this.setState({ value });
  }

  render() {
    const { label, disabled } = this.props;
    const [hours = '', minutes = ''] = this.getInputValues();

    let className = 'time-control ';
    if (disabled)
      className += 'disabled';

    return (
      <div className={className}>
        <label>{label}</label>
        <div className="time-component">
          <span className="icon" onClick={() => this.onTimeChange(-1)}>
            <FontAwesomeIcon icon={['fas', 'minus']} />
          </span>
          <div className="inputs-row">
            <input ref={this.hoursInputRef}
                   className="time-input"
                   type="number"
                   value={hours}
                   onClick={() => this.onInputClick(this.hoursInputRef)}
                   onChange={e => this.onInputChange(e.target.value, TimeKey.Hours)} />
            :
            <input ref={this.minutesInputRef}
                   className="time-input"
                   type="number"
                   value={minutes}
                   onClick={() => this.onInputClick(this.minutesInputRef)}
                   onChange={e => this.onInputChange(e.target.value, TimeKey.Minutes)} />
          </div>
          <span className="icon" onClick={() => this.onTimeChange(1)}>
            <FontAwesomeIcon icon={['fas', 'plus']} />
          </span>
        </div>
      </div>
    );
  }
};