import React from 'react';


type Props = {
  value?: boolean | number;
  onChange?: (value: boolean | number) => void;
};

export class Toggle extends React.Component<Props> {

  onClick = () => {
    const { value, onChange } = this.props;
    onChange?.(!value);
  }

  render() {
    const { value } = this.props;

    let className = 'toggle ';
    if (value)
      className += 'checked ';

    return (
      <div className={className} onClick={this.onClick}>
        <div className="switch" />
      </div>
    );
  }
}
