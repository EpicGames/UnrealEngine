import React from 'react';


type Props = {
  value?: number | string;
  draggable?: boolean;
  min?: number;
  max?: number;
  precision?: number;
  disabled?: boolean;

  onChange: (value: number) => void;
};

type State = {
  value: string;
};

export class ValueInput extends React.Component<Props, State> {

  ref = React.createRef<HTMLDivElement>();
  input = React.createRef<HTMLInputElement>();

  monitoring: boolean = false;
  dragStartX: number = 0;
  touchTime: number = null;

  getValue = (): string => {
    let { value, precision } = this.props;
    if (typeof value === 'string')
      value = Number.parseFloat(value);

    return value?.toFixed?.(precision ?? 2);
  }

  getNumberValue = () => {
    let { value } = this.props;
    if (typeof value === 'string')
      value = Number.parseFloat(value);

    return value;
  }

  state: State = {
    value: this.getValue(),
  };

  componentDidUpdate (prevProps: Props) {
    const { value } = this.props;
    const newValue = this.getValue();

    if (prevProps.value !== value && this.state.value !== newValue)
      this.setState({ value: newValue });
  }

  onContextMenu = (e) => {
    e.preventDefault();
  }

  onClick = () => {
    this.input.current.select();
  }

  onDown = (e: React.PointerEvent<HTMLDivElement>) => {
    if (this.props.draggable === false)
      return;

    this.monitoring = true;
    this.dragStartX = e.clientX;
    
    if (this.touchTime === null || (performance.now() - this.touchTime) > 3000)
      this.touchTime = performance.now();
    this.ref.current?.setPointerCapture(e.pointerId);
  }

  onUp = (e: React.PointerEvent<HTMLDivElement>) => {
    this.monitoring = false;
    this.touchTime = performance.now();
    this.ref.current?.releasePointerCapture(e.pointerId);
  }

  onMove = (e: React.PointerEvent<HTMLDivElement>) => {
    if (!this.monitoring || !this.ref.current)
      return;

    if (this.touchTime !== null) {
      // initial delay of 150ms
      if (performance.now() - this.touchTime < 150)
        return;

      this.dragStartX = e.clientX;
      this.touchTime = null;
    }

    const { min, max } = this.props;

    let value = this.getNumberValue();
    let x = e.clientX - this.dragStartX;
    this.dragStartX = e.clientX;

    if (min !== undefined && max !== undefined) {
      const range = Math.abs(max - min);
      if (range > 0.01 && 2 * range < window.innerWidth)
        x *= range / 50;
    }

    value += x;
    if (min !== undefined)
      value = Math.max(min, value);

    if (max !== undefined)
      value = Math.min(value, max);

    if (isNaN(value))
      value = 0;

    this.props.onChange?.(value);
    e.preventDefault();
  }

  onManualChange = (e: React.ChangeEvent<HTMLInputElement>) => {
    this.setState({ value: e.target.value });
  }

  onKeyDown = (e: React.KeyboardEvent<HTMLInputElement>) => {
    switch (e.key) {
      case 'Enter':
        this.onChangeValue();
        break;

      case 'Escape':
        this.setPropsValue();
        break;
    }
  }

  setPropsValue = () => {
    this.setState({ value: this.getValue() });
  }

  onChangeValue = () => {
    const { min, max } = this.props;
    let value = Number.parseFloat(this.state.value);

    if (!isNaN(value)) {
      if (!isNaN(min))
        value = Math.max(min, value);

      if (!isNaN(max))
        value = Math.min(max, value);

      if (value !== this.props.value)
        this.props.onChange(value);
    } 

    this.setPropsValue();
  }

  render() {
    const { value } = this.state;
    const { disabled } = this.props;

    return (
      <span ref={this.ref}
            className="input-field-container"
            onClick={this.onClick}
            onDoubleClick={this.onClick}
            onPointerDown={this.onDown}
            onPointerMove={this.onMove}
            onPointerUp={this.onUp}
            onLostPointerCapture={this.onUp}>
        <input ref={this.input}
               disabled={disabled}
               className="input-field"
               value={value ?? ''}
               type="number"
               inputMode="numeric"
               pattern="[0-9]*"
               onChange={this.onManualChange}
               onKeyDown={this.onKeyDown}
               onBlur={this.onChangeValue}
               onContextMenu={this.onContextMenu} />
      </span>
    );
  }
}