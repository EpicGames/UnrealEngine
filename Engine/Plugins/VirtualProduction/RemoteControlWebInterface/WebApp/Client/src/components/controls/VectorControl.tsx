import React from 'react';
import { PropertyType } from 'src/shared';
import { WidgetUtilities } from 'src/utilities';
import { ValueInput } from './ValueInput';
import { FontAwesomeIcon } from '@fortawesome/react-fontawesome';
import _ from 'lodash';


type Props = {
  label: string;
  propertyType: PropertyType;
  value: any;
  step: number;
  defaultValue?: any;

  onChange?: (property: string, value: any) => void;
}

type State = {
  step: number;
  activeKey: string;
}

const dashArr = Array.from(Array(81));

export class VectorControl extends React.Component<Props, State> {

  static defaultProps: Props = {
    label: '',
    propertyType: PropertyType.Vector,
    value: {},
    step: 0.1,
  };

  state: State = {
    step: null,
    activeKey: null,
  }

  sliderRef = React.createRef<HTMLDivElement>();
  pointerRef = React.createRef<HTMLDivElement>();
  value = null;
  monitoring: boolean = false;
  offset: number = 0;
  step: number = 0;
  interval = null;

  componentDidMount() {
    const { propertyType } = this.props;
    const keys = WidgetUtilities.getPropertyKeys(propertyType);
    const activeKey = _.first(keys);

    if (!activeKey)
      return;

    this.setState({ activeKey });
  }

  onMouseDown = (e: React.PointerEvent<HTMLDivElement>) => {
    this.monitoring = true;
    this.sliderRef.current.setPointerCapture(e.pointerId);
    this.offset = 0;
    this.value = { ...this.props.value };

    const pointer = this.pointerRef.current.getBoundingClientRect();
    if (pointer.left <= e.clientX && e.clientX <= pointer.right)
      this.offset = e.clientX - (pointer.left + (pointer.width / 2));

    this.onMouseMove(e);
    this.makeMove();

    this.interval = setInterval(this.makeMove, 100);
  }

  onMouseUp = (e: React.PointerEvent<HTMLDivElement>) => {
    if (!this.monitoring)
      return;

    this.monitoring = false;
    this.offset = 0;
    this.value = null;
    this.step = 0;
    this.sliderRef.current.releasePointerCapture(e.pointerId);

    clearInterval(this.interval);
    this.interval = null;

    this.setState({ step: null });
  }

  onMouseMove = (e: React.PointerEvent<HTMLDivElement>) => {
    if (!this.monitoring || !this.sliderRef.current)
      return;

    const slider = this.sliderRef.current;

    const rect = slider.getBoundingClientRect();
    const x = e.clientX - this.offset;
    const center = (rect.right - rect.left) / 2;
    const ratio = Math.max(-1, Math.min(((x - (rect.left + center)) / rect.width) * 2, 1));
    const step = +ratio.toFixed(2);

    this.step = step * this.props.step;
    this.setState({ step });
  }

  onKeySelect = (e: React.MouseEvent, activeKey: string) => {
    e.stopPropagation();
    this.setState({ activeKey });
  }

  makeMove = () => {
    const { activeKey } = this.state;
    if (!activeKey || this.value?.[activeKey] === undefined)
      return;  

    this.value[activeKey] += this.step;
    this.props.onChange(activeKey, this.value[activeKey]);
  }

  resetActiveKey = (e: React.MouseEvent) => {
    this.setState({ activeKey: null });
  }

  getPointerStyles = (): React.CSSProperties => {
    const { step } = this.state;
    const slider = this.sliderRef.current?.getBoundingClientRect?.();

    if (step === null || !slider)
      return { transition: 'transform .2s' };

    const translate = (slider.width / 2 * step).toFixed();
    return { transform: `translateX(${translate}px)` };
  }

  getIsDisabled = () => {
    const { activeKey } = this.state;
    return !activeKey;
  }

  renderPropertyInput = (key: string) => {
    const { activeKey } = this.state;
    const { value } = this.props;

    let className = 'key ';

    if (key === activeKey)
      className += 'active';

    return (
      <div key={key} 
           className={className} 
           onClick={(e) => this.onKeySelect(e, key)}>
        <div className="property">{key}:</div>
        <ValueInput value={value[key] ?? 0}
                    disabled={activeKey !== key}
                    onChange={value => this.props.onChange(key, value)} />
      </div>
    );
  }

  onReset = () => {
    const { onChange, defaultValue } = this.props;
    onChange(null, defaultValue);
  }

  render() {
    const { label, propertyType } = this.props;
    const keys = WidgetUtilities.getPropertyKeys(propertyType);

    let className = 'vector-control ';
    if (this.getIsDisabled())
      className += 'disabled';

    return (
      <div className={className} onClick={this.resetActiveKey}>
        <div className="title">{label}</div>
        <div className="vector-item">
          <div className="keys-row">
            {keys.map(this.renderPropertyInput)}
          </div>
          <div className="vector-container">
            <div ref={this.sliderRef}
                 className="slider-control"
                 onClick={e => e.stopPropagation()}
                 onPointerDown={this.onMouseDown}
                 onLostPointerCapture={this.onMouseUp}
                 onPointerMove={this.onMouseMove}>
              {dashArr.map((_, i) => <div key={`dash-${i}`} className="dash" />)}
            </div>
            <div ref={this.pointerRef} className="pointer" style={this.getPointerStyles()} />
          </div>
        </div>
        <span className="reset-icon" onClick={this.onReset}>
          <FontAwesomeIcon icon={['fas', 'undo']} />
        </span>
      </div>
    );
  }
};