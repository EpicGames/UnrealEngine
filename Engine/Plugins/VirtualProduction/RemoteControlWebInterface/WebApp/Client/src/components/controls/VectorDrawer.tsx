import React from 'react';
import { FontAwesomeIcon } from '@fortawesome/react-fontawesome';
import { ICustomStackProperty, JoystickValue, PropertyType, WidgetType, WidgetTypes } from 'src/shared';
import { DialsWrapper, JoysticksWrapper, SlidersWrapper, DialMode } from '.';
import { WidgetUtilities } from 'src/utilities';
import _ from 'lodash';


type Props = {
  value: JoystickValue;
  widget?: ICustomStackProperty;
  label?: React.ReactNode;
  min?: number;
  max?: number;

  onChange: (value?: JoystickValue) => void;
  onClose: () => void;
};

type State = {
  mode?: WidgetType;
};

const keys = [WidgetTypes.Joystick, WidgetTypes.Dial, WidgetTypes.Sliders];

export class VectorDrawer extends React.Component<Props, State> {
  state : State = {};

  componentDidMount() {
    const { widget } = this.props;
    
    const modes = _.compact(keys.map(key => widget.widgets?.find(w => w === key)));
    const mode = _.first(modes);

    this.setState({ mode });
  }

  componentDidUpdate(prevProps: Props) {
    const { widget } = this.props;
    const { mode } = this.state;

    if (widget !== prevProps.widget) {
      const modes = _.compact(keys.map(key => widget.widgets?.find(w => w === key)));

      if (!modes.length)
        return this.props.onClose();

      if (!modes.find(m => m === mode))
        this.setState({ mode: _.first(modes) });
    }
  }

  renderJoystick = (keys: string[], min: number, max: number) => {
    const { widget, value, onChange } = this.props;
    return (      
      <JoysticksWrapper type={widget?.propertyType} 
                        min={min}
                        max={max}
                        value={value}
                        keys={keys}
                        showReset={false}
                        onChange={onChange} />
    );
  }

  setMode = (mode: WidgetType) => {
    this.setState({ mode });
  }

  renderMode = (mode: string, type: WidgetType) => {
    const { widget } = this.props;
    if (widget.widgets.length < 2 || !widget.widgets.includes(type))
      return null;

    let className = 'mode ';
    if (mode === type)
      className += 'selected ';

    return <div className={className} onClick={() => this.setMode(type)}>{type}</div>;
  }

  render() {
    const { label, value, min, max, widget, onChange, onClose } = this.props;
    if (!widget?.widgets?.length)
      return null;

    let { mode } = this.state;
    if (!mode || !widget.widgets.includes(mode))
      mode = widget.widgets[0];

    const keys = WidgetUtilities.getPropertyKeys(widget?.propertyType);
    const rotator = widget.propertyType === PropertyType.Rotator;
    let dialMode = DialMode.Endless;

    if (min !== undefined && max !== undefined)
      dialMode = DialMode.Range;

    if (rotator)
      dialMode = DialMode.Rotation;

    const showModes = widget.widgets.length > 1;

    return (
      <div className="vector-drawer-wrapper" onClick={e => e.stopPropagation()}>
        <div className="drawer-header">
          <div className="modes header-block">
            {this.renderMode(mode, WidgetTypes.Joystick)}
            {this.renderMode(mode, WidgetTypes.Dial)}
            {this.renderMode(mode, WidgetTypes.Sliders)}
          </div>
          <div className="header-block">
            <span>{label}</span>
            <FontAwesomeIcon className="reset-btn" icon={['fas', 'undo']} onClick={() => onChange?.()} />
          </div>
          <div className="header-block btns">
            <FontAwesomeIcon className="close-btn" icon={['fas', 'times']} onClick={onClose} />
          </div>
        </div>

        <div className={`controls-block ${showModes ? '' : 'controls-only'}`}>
          {mode === WidgetTypes.Joystick && !rotator && 
            this.renderJoystick(keys, min, max)
          }

          {mode === WidgetTypes.Sliders &&
            <SlidersWrapper widget={widget}
                            min={min}
                            max={max}
                            value={value}
                            onChange={onChange} />
          }

          {mode === WidgetTypes.Dial &&
            <DialsWrapper size={widget.propertyType !== PropertyType.Vector2D ? 600 : 400}
                          min={min}
                          max={max}
                          type={widget.propertyType}
                          mode={dialMode}
                          value={value}
                          hidePrecision={true}
                          onChange={onChange} />
          }
        </div>
      </div>
    );
  }
}