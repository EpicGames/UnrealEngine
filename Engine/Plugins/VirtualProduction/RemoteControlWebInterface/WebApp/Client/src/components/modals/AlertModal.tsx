import { FontAwesomeIcon } from '@fortawesome/react-fontawesome';
import React from 'react';
import ReactDOM from 'react-dom';

type Props = {
  text: string;
  close: (isConfirmed?: boolean) => void;
};

export class AlertModal extends React.Component<Props> {

  componentDidMount() {    
    document.addEventListener('keydown', this.onKeyPress);
  }

  componentWillUnmount() {    
    document.removeEventListener('keydown', this.onKeyPress);
  }

  onKeyPress = (e) => {
    if (e.key === 'Escape')
      this.onCancel();
  }

  private static pending: (isConfirmed?: boolean) => void;

  public static show(text: string): Promise<boolean> {
    AlertModal.pending?.(undefined);
    AlertModal.pending = undefined;

    const div = document.createElement('div');
    document.getElementById('modal').prepend(div);

    return new Promise<boolean>(resolve => {
      AlertModal.pending = resolve;

      ReactDOM.render(<AlertModal text={text} close={resolve} />, div);
    })
      .finally(() => {
        const unmountResult = ReactDOM.unmountComponentAtNode(div);
        if (unmountResult && div.parentNode)
          div.parentNode.removeChild(div);
      });
  }

  onOK = () => {
    this.props.close(true);
  }

  onCancel = () => {
    this.props.close(false);
  }

  render() {
    const { text } = this.props;

    return (
      <div className="fullscreen" onClick={this.onCancel}>
        <div className="alert-modal" onClick={e => e.stopPropagation()}>
          <div className="close-modal" onClick={this.onCancel}>
            <FontAwesomeIcon icon={['fas', 'times']} />
          </div>
          <div className="alert-text">{text}</div>
          <div className="footer">
            <button className="btn btn-secondary" onClick={this.onCancel}>Cancel</button>
            <button className="btn btn-secondary" onClick={this.onOK}>Ok</button>
          </div>
          
        </div>
      </div>
    );
  }
}