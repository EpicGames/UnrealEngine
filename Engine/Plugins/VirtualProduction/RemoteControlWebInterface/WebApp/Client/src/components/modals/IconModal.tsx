import { FontAwesomeIcon } from '@fortawesome/react-fontawesome';
import { fas, IconName } from '@fortawesome/free-solid-svg-icons';
import React from 'react';
import ReactDOM from 'react-dom';
import _ from 'lodash';


type Props = {
  selected?: string;
  close: (name?: IconName) => void;
};

type State = {
  filter: string;
  filterText: string;
  selected?: IconName;
};

export class IconModal extends React.Component<Props, State> {
  state: State = {
    filter: '',
    filterText: '',
  };

  componentDidMount() {    
    document.addEventListener('keydown', this.onKeyPress);
  }

  componentWillUnmount() {    
    document.removeEventListener('keydown', this.onKeyPress);
  }

  onKeyPress = (e) => {
    if (e.key === 'Escape')
      this.onCancel();
  }

  private static pending: (name?: IconName ) => void;

  public static show(selected: string): Promise<IconName> {
    IconModal.pending?.(undefined);
    IconModal.pending = undefined;

    const div = document.createElement('div');
    document.getElementById('modal').prepend(div);

    return new Promise<IconName>(resolve => {
      IconModal.pending = resolve;

      ReactDOM.render(<IconModal selected={selected} close={resolve} />, div);
    })
      .finally(() => {
        const unmountResult = ReactDOM.unmountComponentAtNode(div);
        if (unmountResult && div.parentNode)
          div.parentNode.removeChild(div);
      });
  }

  onFilterTextChange = (filterText: string) => {
    this.setState({ filterText });
    this.onFilterChangeDelayed(filterText);
  }

  onFilterChange = (filter: string) => {
    filter = filter.toLocaleLowerCase();
    this.setState({ filter });
  }

  onFilterChangeDelayed = _.debounce(this.onFilterChange, 250);

  onResetFilter = () => {
    this.setState({ filterText: '', filter: '' });
  }

  onSelect = (selected: IconName) => {
    this.setState({ selected });
  }

  onOK = () => {
    const { selected } = this.state;
    if (selected)
      this.props.close(selected);
  }

  onCancel = () => {
    this.props.close();
  }

  onKeyDown = (e: React.KeyboardEvent<HTMLInputElement>) => {
    switch (e.key) {
      case 'Enter':
        return this.onOK();

      case 'Escape':
        return this.onCancel();
    }
  }

  renderIcons = () => {
    const { filter } = this.state;
    const selected = this.state.selected || this.props.selected;

    let keys = Object.keys(fas);
    if (filter)
      keys = keys.filter(key => fas[key].iconName.includes(filter));
    
    return (
      <div className="icon-wrapper">
        {keys.map(key => (
          <div key={fas[key].iconName} className={`icon-block ${fas[key].iconName === selected ? 'selected' : ''}`}>
            <FontAwesomeIcon className="icon" icon={['fas', fas[key].iconName]} onClick={() => this.onSelect(fas[key].iconName)} />
            <div className="icon-text">{fas[key].iconName}</div>
          </div>)
        )}
      </div>
    );
  }

  render() {
    const { filterText, selected } = this.state;

    return (
      <div className="fullscreen" onClick={this.onCancel}>
        <div className="icon-modal" onClick={e => e.stopPropagation()}>
          <div className="close-modal" onClick={this.onCancel}>
            <FontAwesomeIcon icon={['fas', 'times']} />
          </div>
          <div className="search-wrapper search-position">
            <FontAwesomeIcon icon={['fas', 'search']} />
            <input value={filterText}
                    className="icon-search search-input"
                    placeholder="Search icon" 
                    onChange={e => this.onFilterTextChange(e.target.value)}
                    autoFocus={true}
                    onKeyDown={this.onKeyDown}
                    autoComplete="off" />
            {filterText &&
              <FontAwesomeIcon icon={['fas', 'times-circle']} onClick={this.onResetFilter} />
            }
          </div>
          {this.renderIcons()}
          <div className="btn-wrapper">
            <button className="btn btn-secondary" onClick={this.onCancel}>Cancel</button>
            <button className={`btn btn-secondary ${!selected && 'disabled'}`} disabled={!selected} onClick={this.onOK}>Apply</button>
          </div> 
        </div>
      </div>
    );
  }
}