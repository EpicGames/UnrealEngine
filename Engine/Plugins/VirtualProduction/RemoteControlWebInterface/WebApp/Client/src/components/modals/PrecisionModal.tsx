import React from 'react';
import { WidgetUtilities } from 'src/utilities';
import { ValueInput, SliderWheel, Slider, ColorPicker, ColorPickerParts, ColorMode } from '../controls';
import { ColorProperty, IHsvColor, PropertyType, PropertyValue, VectorProperty, WidgetType, WidgetTypes } from 'src/shared';
import { FontAwesomeIcon } from '@fortawesome/react-fontawesome';
import _ from 'lodash';
import { AlertModal } from './AlertModal';


type Props = {
  widget?: WidgetType;
  property: string;
  meta?: Partial<{ type: PropertyType, min: number, max: number, precision: number, alpha: boolean }>;
  type?: PropertyType;
  label?: string | JSX.Element;
  value?: any;

  onClose?: () => void;
  onChange?: (value?: PropertyValue) => void; 
  onMetadataChange?: (key: string, value: string) => void;
};

type State = {
  step: number;
  min: number;
  max: number;
  mode: ColorMode;
  negative: boolean;
};

export class PrecisionModal extends React.Component<Props, State> {
  state: State = {
    step: 1,
    min: 0.01,
    max: 10,
    mode: ColorMode.Rgb,
    negative: false,
  }

  value: number = 0;
  originalValue: number = null;
  hsv: IHsvColor = { h: 0, s: 0, v: 0 };
  
  componentDidMount() {
    const { widget, value } = this.props;

    document.addEventListener('keydown', this.onKeyPress);

    const loaded: any = { max: 10 };
    if (widget === WidgetTypes.ColorPicker)
      loaded.max = 1;

    this.originalValue = value;
    
    try {
      const sensitivity = JSON.parse(localStorage.getItem(this.props.property));

      for (const property in this.state) {
        if (sensitivity[property])
          loaded[property] = sensitivity[property];
      }
    } catch {
    }

    this.setState(loaded);
  }

  componentWillUnmount() {    
    document.removeEventListener('keydown', this.onKeyPress);
  }

  normalizeValue = (value: number, min: number, max: number) => {
    return Math.max(min, Math.min(value, max));
  }

  onKeyPress = (e: KeyboardEvent) => {
    if (e.key === 'Escape')
      this.onCancel();
  }

  onCancel = () => {
    this.props.onClose();
  }

  onKeyDown = (e: React.KeyboardEvent<HTMLInputElement>) => {
    if (e.key === 'Escape')
      return this.onCancel();
  }

  onChange = (factor: number) => {
    const { step } = this.state;
    let { value, meta } = this.props;

    value += step * factor;
    if (meta?.min !== undefined)
      value = Math.max(meta.min, value);
    
    if (meta?.max !== undefined)
      value = Math.min(value, meta.max);

    this.props.onChange?.(value);
  }

  onStepChange = (step: number) => {
    const { min, max } = this.state;

    step = this.normalizeValue(step, min, max);
    this.onSetMeta('step', step);
  }

  onRgbValueInputChange = (key: string, v: number) => {
    const { value } = this.props;
    const { negative } = this.state;
    
    if (!value)
      return;

    if (!negative)
      v = Math.max(0, v);

    value[key] = v;
    this.props.onChange?.(value);
  }

  onHsvValueInputChange = (key: string, v: number) => {
    const { type } = this.props;
    let hsv = this.getHsvColor();

    hsv[key] = v;
    hsv.h /= 360;

    if (type === PropertyType.Color)
      hsv = { ...hsv, s: this.normalizeValue(hsv.s, 0, 1), v: this.normalizeValue(hsv.v, 0, 1) };

    const rgb = WidgetUtilities.hsv2rgb(hsv);

    let max = 1;
    let color = rgb as ColorProperty | VectorProperty;

    switch (type) {
      case PropertyType.Vector4:
        color = WidgetUtilities.rgbToColor(rgb, max);
        break;

      case PropertyType.LinearColor:
        color = WidgetUtilities.rgbToColor(rgb, max, false);
        break;
    }

    this.props.onChange?.(color);
  }

  onWheelStart = () => {
    const { value } = this.props;
    this.value = value;
  }

  onHsvWheelStart = () => {
    const hsv = this.getHsvColor();

    this.hsv = hsv;
  }

  onWheelMove = (sign: number, offset: number) => {
    const { meta } = this.props;
    const { step } = this.state;

    let value = this.value + step * offset;
    if (meta?.min !== undefined)
      value = Math.max(meta.min, value);
    
    if (meta?.max !== undefined)
      value = Math.min(value, meta.max);

    this.props.onChange?.(value);
  }

  onRgbWheelMove = (key: string, sign: number) => {
    const { type, value } = this.props;
    const { negative } = this.state;
    const step = sign * (type === PropertyType.Color ? 10 : 0.01) * this.state.step;

    value[key] += step;

    if (type === PropertyType.Color)
      value[key] = this.normalizeValue(value[key], 0, 255);

    if (!negative)
      value[key] = Math.max(0, value[key]);

    this.props.onChange(value);
  }

  onHsvWheelMove = (key: string, sign: number, step: number, min?: number, max?: number) => {
    const { type } = this.props;

    step *= sign * this.state.step;

    this.hsv[key] += step;
    if (min !== undefined)
      this.hsv[key] = Math.max(min, this.hsv[key]);
    if (max !== undefined)
      this.hsv[key] = Math.min(this.hsv[key], max);

    let offset = 0;
    if (this.hsv.h < 0)
      offset = 360;

    if (type === PropertyType.Color)
      this.hsv = { ...this.hsv, s: this.normalizeValue(this.hsv.s, 0, 1), v: this.normalizeValue(this.hsv.v, 0, 1) };

    this.hsv = { ...this.hsv, h: offset + this.hsv.h, s: this.normalizeValue(this.hsv.s, 0, 1) };
    const hsv = { ...this.hsv, h: (this.hsv.h / 360) };
    const color = this.hsvToColor(hsv);

    this.props.onChange?.(color);
  }

  onRgbStepChange = (factor: number, key: string) => {
    const { value, type } = this.props;
    const { negative } = this.state;

    value[key] += factor;

    if (type === PropertyType.Color)
      value[key] = this.normalizeValue(value[key], 0, 255);

    if (!negative)
      value[key] = Math.max(0, value[key]);

    this.props.onChange?.(value);
  }

  onHsvStepChange = (factor: number, key: string, min?: number, max?: number) => {
    let hsv = this.getHsvColor();
    let offset = 0;

    hsv[key] += factor;
    if (min !== undefined)
      this.hsv[key] = Math.max(min, this.hsv[key]);
    if (max !== undefined)
      this.hsv[key] = Math.min(this.hsv[key], max);

    if (hsv.h < 0)
      offset = 1;

    if (this.props.type === PropertyType.Color)
      hsv.v = this.normalizeValue(hsv.v, 1, 0);

    hsv = { ...hsv, h: offset + hsv.h / 360, s: this.normalizeValue(hsv.s, 1, 0) };

    const color = this.hsvToColor(hsv);
    this.props.onChange?.(color);
  }

  getHsvColor = () => {
    const { value, type } = this.props;
    let rgb = value as ColorProperty;

    if (type === PropertyType.Vector4 || type === PropertyType.LinearColor) {
      let max = 0;

      for (const key in value)
        if (value[key] > max)
          max = value[key];

      rgb = WidgetUtilities.colorToRgb(value, max);
    }

    return this.getHsvValue(rgb);
  }

  hsvToColor = (hsv: IHsvColor) => {
    const { type } = this.props;
    const rgb = WidgetUtilities.hsv2rgb(hsv);

    let max = 1;
    let color = rgb as ColorProperty | VectorProperty;

    switch (type) {
      case PropertyType.Vector4:
        color = WidgetUtilities.rgbToColor(rgb, max);
        break;

      case PropertyType.LinearColor:
        color = WidgetUtilities.rgbToColor(rgb, max, false);
        break;
    }

    return color;
  }

  getInputKey = (type: keyof ColorProperty) => {
    let key = type as keyof VectorProperty | keyof ColorProperty;

    switch (this.props.type) {
      case PropertyType.Vector:
      case PropertyType.Vector4:
        switch (type) {
          case 'R':
            key = 'X';
            break;

          case 'G':
            key = 'Y';
            break;

          case 'B':
            key = 'Z';
            break;
        }
        break;
    }

    return key;
  }

  getHsvValue = (rgb: ColorProperty): IHsvColor => {
    const { type } = this.props;
    const value = { ...this.props.value };
    const hsv = WidgetUtilities.rgb2Hsv(rgb);

    delete value.A;
    delete value.W;
    const values = Object.values(value) as number[];

    if (type !== PropertyType.Color)
      hsv.v = Math.max(0, ...values);

    return { ...hsv, h: hsv.h * 360, s: Math.max(0.0001, hsv.s) };
  }

  renderRgbInput = (type: keyof ColorProperty) => {
    const { value } = this.props;
    const key = this.getInputKey(type);
    let precision = 3;
    let min: number;
    let max: number;
    let { step } = this.state;

    if (this.props.type === PropertyType.Color) {
      precision = 0;
      min = 0;
      max = 255;
      step *= 50;
    }

    return (
      <>
        <ValueInput value={value?.[key]}
                    draggable={false}
                    precision={precision}
                    min={min}
                    max={max}
                    onChange={v => this.onRgbValueInputChange(key, v)} />
        <FontAwesomeIcon icon={['fas', 'angle-left']} 
                         className="precision-icon"
                         onClick={() => this.onRgbStepChange(-step, key)} />
        <SliderWheel onWheelMove={sign => this.onRgbWheelMove(key, sign)} />
        <FontAwesomeIcon icon={['fas', 'angle-right']} 
                         className="precision-icon"
                         onClick={() => this.onRgbStepChange(step, key)} />
      </>
    );
  }

  renderHsvInput = (value: number, key: string, multiplier: number = 0.1, min?: number, max?: number) => {
    const step = multiplier * this.state.step;
    let precision = 3;
    if (this.props.type === PropertyType.Color) {
      precision = (key === 'h' ? 0 : 2);
    }

    return (
      <>
        <div className="slider-label">{key.toUpperCase()}:</div>
        <ValueInput value={value}
                    min={min}
                    max={max}
                    draggable={false}
                    precision={precision}
                    onChange={v => this.onHsvValueInputChange(key, v)} />
        <FontAwesomeIcon icon={['fas', 'angle-left']} className="precision-icon" onClick={() => this.onHsvStepChange(-step, key, min, max)} />
        <SliderWheel onWheelStart={this.onHsvWheelStart} onWheelMove={sign => this.onHsvWheelMove(key, sign, step, min, max)} />
        <FontAwesomeIcon icon={['fas', 'angle-right']} className="precision-icon" onClick={() => this.onHsvStepChange(step, key, min, max)} />
      </>
    );
  }

  renderMinMaxSlider = () => {
    const { meta, value } = this.props;

    if (meta?.min === undefined || meta?.max === undefined)
      return null;

    return (
      <div className="row top">
        <ValueInput value={meta.min}
                    onChange={value => this.props.onMetadataChange('Min', String(value))} />
        <Slider value={value}
                min={meta?.min}
                max={meta?.max}
                onChange={value => this.props.onChange(value)}
                precision={meta?.precision}
                showLabel={false} />
        <ValueInput value={meta.max}
                    onChange={value => this.props.onMetadataChange('Max', String(value))} />
      </div>
    );
  }

  onSetMeta = (property: keyof State, value: any) => {
    const update: any = {};
    update[property] = value;
    this.setState(update, () => localStorage.setItem(this.props.property, JSON.stringify(this.state)));
  }

  onRevertChanges = async () => {
    if (!await AlertModal.show('Revert changes?'))
      return;

    this.props.onChange(this.originalValue);
  }

  renderNumberPrecicionModal = () => {
    const { meta, value } = this.props;
    const { step, min, max } = this.state;

    const stepSize = Math.abs(max - min) * 0.1;
    const nextStep = step + stepSize;
    const backStep = step - stepSize <= stepSize ? step / 2 : step - stepSize;

    const isDisabled = _.isEqual(this.originalValue, value);

    return (
      <div className="main">
        {this.renderMinMaxSlider()}
        <div className="row">
          <FontAwesomeIcon icon={['fas', 'backward']}
                           className="precision-icon"
                           onClick={() => this.onChange(-3)} />
          <FontAwesomeIcon icon={['fas', 'angle-double-left']}
                           className="precision-icon"
                           onClick={() => this.onChange(-1)} />
          <FontAwesomeIcon icon={['fas', 'angle-left']}
                           className="precision-icon"
                           onClick={() => this.onChange(-0.2)} />
          <ValueInput min={meta?.min}
                      max={meta?.max}
                      precision={meta?.precision}
                      value={value}
                      onChange={this.props.onChange} />
          <FontAwesomeIcon icon={['fas', 'angle-right']}
                           className="precision-icon"
                           onClick={() => this.onChange(0.2)} />
          <FontAwesomeIcon icon={['fas', 'angle-double-right']}
                           className="precision-icon"
                           onClick={() => this.onChange(1)} />
          <FontAwesomeIcon icon={['fas', 'forward']}
                           className="precision-icon"
                           onClick={() => this.onChange(3)} />
        </div>
        <div className="row">
          <SliderWheel onWheelStart={this.onWheelStart} onWheelMove={this.onWheelMove} />
        </div>
        <div className="sensitivity-block">
          <div className="step-label">Sensitivity</div>
          <div className="row m-0">
            <Slider value={step}
                    min={min}
                    max={max}
                    precision={meta?.precision}
                    showLabel={false}
                    onChange={step => this.onSetMeta('step', step)} />
          </div>
          <div className="row space">
            <div className="limits">
              Min
            <ValueInput value={min}
                        draggable={false}
                        min={0.01}
                        onChange={min => this.onSetMeta('min', Math.min(min, max))} />
            </div>
            <div className="step-input">
              <FontAwesomeIcon icon={['fas', 'backward']} className="icon" onClick={() => this.onStepChange(backStep)} />
              <ValueInput value={step}
                          draggable={false}
                          onChange={this.onStepChange} />
              <FontAwesomeIcon icon={['fas', 'forward']} className="icon" onClick={() => this.onStepChange(nextStep)} />
            </div>
            <div className="limits">
              Max
            <ValueInput value={max}
                        min={0.01}
                        draggable={false}
                        onChange={max => this.onSetMeta('max', Math.max(min, max))} />
            </div>
          </div>
        </div>
        <button className="btn btn-revert"
                disabled={isDisabled}
                onClick={this.onRevertChanges}>Revert Changes</button>
      </div>
    );
  }

  renderColorPickerModal = () => {
    const { mode, step, negative } = this.state;
    const { value, type, meta } = this.props;
    const hsv = this.getHsvColor();

    let rgbBtnClassName = 'btn ';
    let hsvBtnClassName = 'btn ';

    const isDisabled = _.isEqual(this.originalValue, value);

    switch (mode) {
      case ColorMode.Rgb:
        rgbBtnClassName += 'btn-primary';
        hsvBtnClassName += 'btn-secondary';
        break;

      case ColorMode.Hsv:
        rgbBtnClassName += 'btn-secondary';
        hsvBtnClassName += 'btn-primary';
        break;
    }

    return (
      <div className="main color-picker">
        <div className="row m-0">
          {type !== PropertyType.Color &&
            <label className="toggle-checkbox">
              <input type="checkbox" checked={negative} onChange={e => this.onSetMeta('negative', e.target.checked)} />
              <span className="toggle-label">Allow Negative Values</span>
            </label>
          }
          <div className="btn-group">
            <button className={rgbBtnClassName}
                    onClick={() => this.setState({ mode: ColorMode.Rgb })}>
              <FontAwesomeIcon icon={['fas', 'circle']} />
            </button>
            <button className={hsvBtnClassName}
                    onClick={() => this.setState({ mode: ColorMode.Hsv })}>
              <FontAwesomeIcon icon={['fas', 'square']} />
            </button>
          </div>
        </div>
        <div className="row start space">
          <div className="slider-wheel-rows">
            <div className="slider-wheel-row">
              <div className="slider-label">R:</div>
              {this.renderRgbInput('R')}
            </div>
            <div className="slider-wheel-row">
              <div className="slider-label">G:</div>
              {this.renderRgbInput('G')}
            </div>
            <div className="slider-wheel-row">
              <div className="slider-label">B:</div>
              {this.renderRgbInput('B')}
            </div>
            {!!meta?.alpha &&
              <div className="slider-wheel-row">
                <div className="slider-label">A:</div>
                {this.renderRgbInput('A')}
              </div>}
            <div className="slider-wheel-row">
              {this.renderHsvInput(hsv?.h, 'h', 3)}
            </div>
            <div className="slider-wheel-row">
              {this.renderHsvInput(hsv?.s, 's', 0.1, 0.0001, 1)}
            </div>
            <div className="slider-wheel-row">
              {this.renderHsvInput(hsv?.v, 'v', 0.1, 0.0001)}
            </div>
            <div className="sensitivity-slider">
              <div className="step-label">Sensitivity</div>
              <div className="row m-0">
                <Slider value={step}
                        min={0.1}
                        max={1}
                        showLabel={false}
                        onChange={step => this.onSetMeta('step', step)} />
              </div>
            </div>
          </div>
          <div className="color-container">
            <ColorPicker value={value}
                         parts={[ColorPickerParts.Circle]}
                         type={type}
                         widget={WidgetTypes.ColorPicker}
                         mode={mode}
                         alpha={!!meta?.alpha}
                         onChange={this.props.onChange} />
            <button className="btn btn-revert"
                    disabled={isDisabled}
                    onClick={this.onRevertChanges}>Revert Changes</button>
          </div>
        </div>
      </div>
    );
  }

  renderPrecisionModalContent = () => {
    const { widget } = this.props;

    switch (widget) {
      case WidgetTypes.ColorPicker:
      case WidgetTypes.MiniColorPicker:
        return this.renderColorPickerModal();

      default:
        return this.renderNumberPrecicionModal();
    }
  }

  render() {
    const { label } = this.props;

    return (
      <div className="fullscreen precision" onClick={this.onCancel}>
        <div className="precision-modal" onClick={e => e.stopPropagation()}>
          <div className="close-modal" onClick={this.onCancel}>
            <FontAwesomeIcon icon={['fas', 'times']} />
          </div>
          <div className="label">
            <p>{label}</p>
          </div>
          {this.renderPrecisionModalContent()}
        </div>
      </div>
    );
  }
}