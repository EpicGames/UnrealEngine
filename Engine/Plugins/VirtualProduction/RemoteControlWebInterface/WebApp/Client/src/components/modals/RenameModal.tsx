import { FontAwesomeIcon } from '@fortawesome/react-fontawesome';
import React from 'react';
import ReactDOM from 'react-dom';


type Props = {
  name: string;
  placeholder: string;
  close: (name?: string) => void;
};

type State = {
  name: string;
};

export class RenameModal extends React.Component<Props, State> {
  state: State = {
    name: '',
  };

  componentDidMount() {    
    document.addEventListener('keydown', this.onKeyPress);
    
    const { name } = this.props;
    this.setState({ name });
  }

  componentWillUnmount() {    
    document.removeEventListener('keydown', this.onKeyPress);
  }

  onKeyPress = (e) => {
    if (e.key === 'Escape')
      this.onCancel();
  }

  private static pending: (name?: string) => void;

  public static rename(name: string, placeholder: string): Promise<string> {
    RenameModal.pending?.(undefined);
    RenameModal.pending = undefined;

    const div = document.createElement('div');
    document.getElementById('modal').prepend(div);

    return new Promise<string>(resolve => {
      RenameModal.pending = resolve;

      ReactDOM.render(<RenameModal name={name} placeholder={placeholder} close={resolve} />, div);
    })
      .finally(() => {
        const unmountResult = ReactDOM.unmountComponentAtNode(div);
        if (unmountResult && div.parentNode)
          div.parentNode.removeChild(div);
      });
  }

  onChange = (e: React.ChangeEvent<HTMLInputElement>) => {
    const name = e.target.value;
    this.setState({ name });
  }

  onOK = () => {
    const { name } = this.state;
    this.props.close(name);
  }

  onCancel = () => {
    this.props.close();
  }

  onKeyDown = (e: React.KeyboardEvent<HTMLInputElement>) => {
    switch (e.key) {
      case 'Enter':
        return this.onOK();

      case 'Escape':
        return this.onCancel();
    }
  }

  render() {
    const { close, placeholder } = this.props;
    const { name } = this.state;

    return (
      <div className="fullscreen" onPointerDown={this.onCancel}>
        <div className="rename-modal" onPointerDown={e => e.stopPropagation()}>
          <div className="close-modal" onClick={this.onCancel}>
            <FontAwesomeIcon icon={['fas', 'times']} />
          </div>
          <input value={name ?? ''}
                  placeholder={placeholder}
                  onChange={this.onChange}
                  autoFocus={true}
                  onKeyDown={this.onKeyDown}
                  autoComplete="off"
                  dir="auto" />
          <div className="buttons-wrapper">
            <button className="btn btn-secondary" onClick={this.onCancel}>Cancel</button>
            <button className="btn btn-secondary" onClick={() => close(name)}>Rename</button>
          </div>
        </div>
      </div>
    );
  }
}