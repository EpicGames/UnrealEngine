import React from 'react';
import ReactDOM from 'react-dom';
import { IAsset } from 'src/shared';
import { _api } from '../../reducers';
import { FontAwesomeIcon } from '@fortawesome/react-fontawesome';
import { SafeImage } from '../controls';
import _ from 'lodash';


type Props = {
  placeholder?: string;
  types?: string[];
  prefix?: string;
  count?: number;
  fallback?: string;

  close: (asset?: IAsset) => void;
};

type State = {
  search: string;
  assets: IAsset[];
  loading: boolean;
};

export class SearchModal extends React.Component<Props, State> {
  state: State = {
    search: '',
    assets: [],
    loading: false,
  };

  componentDidMount() {
    document.addEventListener('keydown', this.onKeyPress);
    this.fetchAssets();
  }

  componentWillUnmount() {
    document.removeEventListener('keydown', this.onKeyPress);
  }

  fetchAssets = async () => {
    const { search } = this.state;
    const { types, prefix, count } = this.props;

    this.setState({ loading: true });
    
    const assets = await _api.assets.search(search, types, prefix, {}, count);
    this.setState({ assets, loading: false });
  }

  fetchAssetsDebounced = _.debounce(this.fetchAssets, 200);

  onKeyPress = (e) => {
    if (e.key === 'Escape')
      this.onCancel();
  }

  private static pending: (asset?: IAsset) => void;

  public static open(props: Partial<Props>): Promise<IAsset> {
    SearchModal.pending?.(undefined);
    SearchModal.pending = undefined;

    const div = document.createElement('div');
    document.getElementById('modal').prepend(div);

    return new Promise<IAsset>(resolve => {
      SearchModal.pending = resolve;
      ReactDOM.render(<SearchModal {...props} close={resolve} />, div);
    })
      .finally(() => {
        const unmountResult = ReactDOM.unmountComponentAtNode(div);
        if (unmountResult && div.parentNode)
          div.parentNode.removeChild(div);
      });
  }

  onOK = () => {    
    this.props.close();
  }

  onCancel = () => {
    this.props.close();
  }

  onKeyDown = (e: React.KeyboardEvent<HTMLInputElement>) => {
    switch (e.key) {
      case 'Enter':
        return this.onOK();

      case 'Escape':
        return this.onCancel();
    }
  }

  onSearch = (search: string) => {
    this.setState({ search });
    this.fetchAssetsDebounced();
  }

  renderAssetItem = (asset: IAsset, key: number) => {
    const { fallback } = this.props;
    const url = _api.assets.thumbnailUrl(asset.Path);

    return (
      <div className="search-item" key={key} onClick={() => this.props.close(asset)}>
        <div className="image">
        <SafeImage src={url} 
                   fallback={fallback ?? "/images/assets/Asset.png"}
                   alt={asset.Name}
                   className="box-image" />
        </div>
        <div className="title" title={asset.Name}>
          {asset.Name}
        </div>
        <div className="path" title={asset.Path}>
          {asset.Path}
        </div>
      </div>
    );
  }

  render() {
    const { placeholder } = this.props;
    const { search, assets, loading } = this.state;

    return (
      <div className="fullscreen" onPointerDown={this.onCancel}>
        <div className="search-modal" onPointerDown={e => e.stopPropagation()}>
          <div className="close-modal" onClick={this.onCancel}>
            <FontAwesomeIcon icon={['fas', 'times']} />
          </div>
          <div className="search-wrapper">
            <FontAwesomeIcon icon={['fas', 'search']} />
            <input value={search}
                   className="icon-search search-input"
                   placeholder={placeholder}
                   onChange={e => this.onSearch(e.target.value)}
                   autoFocus={true}
                   autoComplete="off" />
            {search && <FontAwesomeIcon icon={['fas', 'times-circle']}
                                        className="reset-search-icon"
                                        onClick={() => this.setState({ search: '' })} />}
          </div>
          <div className="search-content">
            {loading &&
              <div className="content-placeholder">
                Searching
                <FontAwesomeIcon icon={['fas', 'sync-alt']}
                                 spin
                                 className="loader-icon" />
              </div>}

            {!loading && !assets.length &&
              <div className="content-placeholder">
                No Data
              </div>}

            {assets.map(this.renderAssetItem)}
          </div>
        </div>
      </div>
    );
  }
}