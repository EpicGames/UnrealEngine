import React from 'react';
import { WidgetProperties } from 'src/utilities';

import './style.less';

/*
  There are 2 ways to create custom widgets:
  1. As all other "built-in" widgets
      1) Add a new enum entry to the WidgetTypes enum
      2) Offer new WidgetType to the compatible property types in WidgetUtilities::compatibleWidgets
      3) Add a case in Widget.tsx to render your new widget
      4) if needed add configuration parameters in WidgetConfig.tsx

  2. Self contained widget 
      By registering a React component (class or a function component), see example bellow:

*/


// Uncomment bellow to register this custom widget example
//
//
// WidgetUtilities.registerWidget(
//   'MyCustomWidget',                            // Name of this custom widget
//   [PropertyType.Float, PropertyType.Double],   // a list of supported property types
//   props => <CustomWidget {...props} />         // render function
// );

export class CustomWidget extends React.Component<WidgetProperties<number>> {

  renderButton = (value: number, text: string) => {
    let className = 'btn ';
    if (Math.abs(value - this.props.value) < 0.01)
      className += 'btn-primary ';
    else
      className += 'btn-secondary ';

    return (
      <button className={className} onClick={() => this.props.onChange?.(value)}>{text}</button>
    );
  }

  render () {

    return (
      <div className="custom-wrapper">
        {this.renderButton(0, 'Off')}
        {this.renderButton(0.5, 'Semi')}
        {this.renderButton(1, 'On')}
      </div>
    );
  }
}