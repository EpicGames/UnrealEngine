.custom-wrapper {
  width: 100%;
  display: flex;
  flex-direction: row;
  flex-wrap: wrap;
  justify-content: center;

  button {
    min-width: 65px;
    padding: 10px;
    margin: 5px;
  }
}