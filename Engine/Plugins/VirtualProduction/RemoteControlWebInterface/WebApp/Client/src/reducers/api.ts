import { Dispatch } from 'redux';
import { createAction, createReducer } from 'redux-act';
import dotProp from 'dot-prop-immutable';
import io from 'socket.io-client';
import { IAsset, IPayload, IPayloads, IPreset, IView, PropertyValue, TabLayout, IHistory, ConnectionSignal } from '../shared';
import _ from 'lodash';


export type ApiState = {
  presets: { [id: string]: IPreset };
  preset?: string;
  payload: IPayload;
  payloads: IPayloads;
  view: IView;
  status: {
    connected?: boolean;
    keyCorrect?: boolean;
    loading?: boolean;
    isOpen?: boolean;
    signal?: ConnectionSignal;
  },
  lockedUI: boolean;
  undo: IHistory[],
  redo: IHistory[],
};


let _preset;
let _dispatch: Dispatch;
let _getState: () => { api: ApiState };
let _socket: SocketIOClient.Socket;
let _passphrase: string;
let _pingInterval: NodeJS.Timeout;

const _host = (process.env.NODE_ENV === 'development' ? `http://${window.location.hostname}:7001` : '');

function _initialize(dispatch: Dispatch, getState: () => { api: ApiState }) {
  _dispatch = dispatch;
  _getState = getState;
  _passphrase = localStorage.getItem('passphrase'); 

  _socket = io(`${_host}/`, { path: '/api/io' });

  _socket
    .on('disconnect', () => {
      dispatch(API.STATUS({ connected: false, isOpen: false, keyCorrect: false, version: undefined }));
      clearInterval(_pingInterval);
    })
    .on('presets', (presets: IPreset[]) => dispatch(API.PRESETS(presets)))
    .on('payloads', (payloads: IPayloads) => {
      dispatch(API.PAYLOADS(payloads));
      if (!_preset || !payloads[_preset])
        return;

      dispatch(API.PAYLOAD(payloads[_preset]));
    })
    .on('value', (preset: string, property: string, value: PropertyValue) => {
      dispatch(API.PAYLOADS_VALUE({ [preset]: { [property]: value }}));

      if (_preset === preset)
        dispatch(API.PAYLOAD({ [property]: value }));
    })
    .on('values', (preset: string, changes: { [key: string]: PropertyValue }) => {
      dispatch(API.PAYLOADS_VALUE({ [preset]: changes }));
      if (_preset === preset)
        dispatch(API.PAYLOAD(changes));
    })
    .on('view', (preset: string, view: IView) => {
      if (_preset !== preset)
        return;

      dispatch(API.VIEW(view));
    })
    .on('connected', (connected: boolean, version: string) => {
      dispatch(API.STATUS({ connected, version }));
      if (_pingInterval)
        clearInterval(_pingInterval);
      
      _pingInterval = null;
      if (_passphrase !== null)
        _api.passphrase.login(_passphrase);
    })
    .on('opened', (isOpen: boolean) => {
      dispatch(API.STATUS({ isOpen, loading: false }));

      _api.presets.get();
      _api.payload.all();
      _pingInterval = setInterval(_api.ping, 1000);
    })
    .on('passphrase', (wrongPassphrase: string) => {
      const isLoginStillCorrect = _passphrase !== wrongPassphrase && _passphrase !== undefined;
      dispatch(API.STATUS({ keyCorrect: isLoginStillCorrect }));
      if (!isLoginStillCorrect)
        _passphrase = undefined;
    })
    .on('loading', (loading: boolean) => {
      dispatch(API.STATUS({ loading }));
    })
    .on('pong', (pingTime: number, pongTime: number) => {
      let signal = ConnectionSignal.Good;
      const diff = pongTime - pingTime;

      if (diff > 50)
        signal = ConnectionSignal.Bad;
      else if (diff > 15)
        signal = ConnectionSignal.Normal;


      dispatch(API.STATUS({ signal }));
    });
}

type IRequestCallback = Function | string | undefined;

async function _request(method: string, url: string, body: string | object | undefined, callback: IRequestCallback, passphrase?: string): Promise<any> {
  const request: RequestInit = { method, mode: 'cors', redirect: 'follow', headers: {} };
  if (body instanceof FormData || typeof(body) === 'string') {
    request.body = body;
  } else if (typeof(body) === 'object') {
    request.body = JSON.stringify(body);
    request.headers['Content-Type'] = 'application/json';
  }
  request.headers['passphrase'] = passphrase ?? _passphrase;

  const res = await fetch(_host + url, request);

  let answer: any = await res.text();
  if (answer.length > 0)
    answer = JSON.parse(answer);
  
  if (res.status === 401)
    _dispatch(API.STATUS(answer));

  if (!res.ok)
    throw answer;

  if (typeof (callback) === 'function')
    _dispatch(callback(answer));

  return answer;
}

function _get(url: string, callback?: IRequestCallback, passphrase?: string)        { return _request('GET', url, undefined, callback, passphrase) };
function _put(url: string, body: any, passphrase?: string)                          { return _request('PUT', url, body, undefined, passphrase) };

const API = {
  STATUS: createAction<any>('API_STATUS'),
  PRESETS: createAction<IPreset[]>('API_PRESETS'),
  PRESET_SELECT: createAction<string>('API_PRESET_SELECT'),
  VIEW: createAction<IView>('API_VIEW'),
  PAYLOAD: createAction<IPayload>('API_PAYLOAD'),
  PAYLOADS: createAction<IPayloads>('API_PAYLOADS'),
  PAYLOADS_VALUE: createAction<IPayloads>('API_PAYLOADS_VALUE'),
  LOCK_UI: createAction<boolean>('LOCK_UI'),
  HISTORY_UPDATE: createAction<{ undo: IHistory[], redo: IHistory[] }>('HISTORY_UPDATE'),
};


export const _api = {
  initialize: () => _initialize.bind(null),
  ping: () => _socket.emit('client', new Date().getTime()),
  lockUI: (locked: boolean) => _dispatch(API.LOCK_UI(locked)),
  undoHistory: () => {
    const { undo, redo, payload } = _getState().api;
    const action = undo.shift();

    if (!action)
      return;
    
    redo.push({ property: action.property, value: payload[action.property], time: new Date() });

    _dispatch(API.HISTORY_UPDATE({ undo, redo }));
    _api.payload.set(action.property, action.value, false);
  },
  redoHistory: () => {
    const { undo, redo, payload } = _getState().api;
    const action = redo.pop();

    if (!action)
      return;

    undo.unshift({ property: action.property, value: payload[action.property], time: new Date() });

    _dispatch(API.HISTORY_UPDATE({ undo, redo }));
    _api.payload.set(action.property, action.value, false);
  },

  presets: {
    get: (): Promise<IPreset[]> => _get('/api/presets', API.PRESETS),
    load: (id: string): Promise<IPreset> => _get(`/api/presets/${id}/load`),
    favorite: (id: string, value: boolean): Promise<IPreset> => _put(`/api/presets/${id}/favorite`, { value }),
    select: (preset?: IPreset) => {
      _api.presets.load(preset?.ID);

      _dispatch(API.PRESET_SELECT(preset?.ID));
      _dispatch(API.HISTORY_UPDATE({ undo: [], redo: [] }));
    },
  },
  views: {
    get: async(preset: string): Promise<IView> => {
      let view = await _get(`/api/presets/view?preset=${preset}`);
      if (typeof(view) !== 'object' || !view?.tabs?.length) {
        view = {
          tabs: [{ name: 'Tab 1', layout: TabLayout.Empty }]
        };
      }

      _dispatch(API.VIEW(view));

      return view;
    },
    set: (view: IView) => {
      _socket.emit('view', _preset, view);
    },
  },
  passphrase: {
    login: async(passphrase: string): Promise<boolean> => {
      const ok = await _get('/api/passphrase', API.STATUS, passphrase);
      if (!ok)
        return false;

      localStorage.setItem('passphrase', passphrase);
      _passphrase = passphrase;
      _api.presets.get();
      _api.payload.all();
      return true;
    },
  },
  payload: {
    get: (preset: string): Promise<IPayload> => _get(`/api/presets/payload?preset=${preset}`, API.PAYLOAD),
    all: (): Promise<IPayloads> => _get('/api/payloads', API.PAYLOADS),
    set: (property: string, value: PropertyValue, historyPush = true) => {
      let { undo, redo, payloads } = _getState().api;

      if (historyPush) {
        if (undo.length > 50)
          undo.shift();

        if (redo.length > 0)
          redo = [];

        const lastUndo = _.last(undo);
        const time = new Date();

        if (lastUndo?.property === property && (time.valueOf() - lastUndo?.time?.valueOf()) < 60 * 1000)
          lastUndo.value = payloads[_preset][property];
        else
          undo.unshift({ property, value: payloads[_preset][property], time });

          _dispatch(API.HISTORY_UPDATE({ undo, redo }));
      }

      _socket.emit('value', _preset, property, value);
    },
    execute: (func: string, args?: Record<string, any>) => {
      _socket.emit('execute', _preset, func, args ?? {});
    },
    metadata: (property: string, meta: string, value: string) => {
      _socket.emit('metadata', _preset, property, meta, value);
    },
    rebind: (properties: string[], owner: string) => {
      _socket.emit('rebind', _preset, properties, owner);
    },
  },
  assets: {
    search: (q: string, types: string[], prefix: string, filterArgs = {}, count: number = 50): Promise<IAsset[]> => {
      return new Promise(resolve => _socket.emit('search', q, types, prefix, filterArgs, count, resolve));
    },
    thumbnailUrl: (asset: string) => `${_host}/api/thumbnail?asset=${asset}`,
  },
  proxy: {
    get: (url: string) => _put('/api/proxy', { method: 'GET', url }),
    put: (url: string, body: any) => _put('/api/proxy', { method: 'PUT', url, body }),
    function: (objectPath: string, functionName: string, parameters: Record<string, any> = {}): Promise<any> => {
      const body = { objectPath, functionName, parameters, generateTransaction: true };
      return _api.proxy.put('/remote/object/call', body)
                        .then(ret => ret?.ReturnValue);
    },
    property: {
      get: <T = {}>(objectPath: string, propertyName?: string): Promise<T> => {
        const body = { 
          objectPath,
          propertyName,
          access: 'READ_ACCESS',
        };
        return _api.proxy.put('/remote/object/property', body);        
      },
      set: (objectPath: string, propertyName: string, propertyValue: any) => {
        const body = { 
          objectPath,
          propertyName,
          propertyValue: { [propertyName]: propertyValue },
          access: 'WRITE_ACCESS',
        };
        return _api.proxy.put('/remote/object/property', body);        
      }
    }
  }
};

const initialState: ApiState = {
  presets: {},
  payload: {},
  payloads: {},
  view: { tabs: null },
  status: {
    connected: false,
    signal: ConnectionSignal.Good,
    keyCorrect: false,
    isOpen: false,
  },
  lockedUI: false,
  undo: [],
  redo: [],
};

const reducer = createReducer<ApiState>({}, initialState);

reducer
  .on(API.STATUS, (state, status) => {
    return dotProp.merge(state, 'status', status);
  })
  .on(API.PRESETS, (state, presets) => {
    const presetsMap = _.keyBy(presets, 'ID');
    state = dotProp.set(state, 'presets', presetsMap);

    let { preset } = state;

    // Is loaded preset still available?
    if (preset && !presetsMap[preset])
      preset = undefined;

    // If there isn't a loaded preset
    if (!preset) {
      // 1. Load the preset name specified in the url
      const params = new URLSearchParams(window.location.search);
      preset = params.get('preset');

      // 2. Load last used preset
      if (!preset || !presetsMap[preset])
        preset = localStorage.getItem('preset');

      // 3. Load first preset
      if (!preset || !presetsMap[preset])
        preset = presets[0]?.ID;

      // No available preset
      if (!preset) {
        _preset = undefined;
        return { ...state, preset: undefined, view: { tabs: null }, payload: {} };
      }

      _preset = preset;

      // Delay loading the preset so it happens after we pull the initial list of presets.
      //If it happens intertwined with or before these requests, the result will get overwritten with an empty preset since we haven't loaded it.
      setTimeout(() => {
        _api.presets.load(preset)
          .then(() => Promise.all([
              _api.views.get(preset),
              _api.payload.get(preset),
          ]));
      },200);

      return { ...state, preset, view: { tabs: [] }, payload: {} };
    }

    return state;
  })
  .on(API.VIEW, (state, view) => dotProp.merge(state, 'view', view))
  .on(API.PAYLOADS, (state, payloads) => ({ ...state, payloads }))
  .on(API.PAYLOADS_VALUE, (state, payloads) => {
    for (const preset in payloads) {
      const payload = payloads[preset];
      for (const property in payload) {
        state = dotProp.set(state, ['payloads', preset, property], payload[property]);
      }
    }
    
    return state;
  })
  .on(API.PAYLOAD, (state, payload) => {
    for (const property in payload)
      state = dotProp.set(state, ['payload', property], payload[property]);
    
    return state;
  })
  .on(API.PRESET_SELECT, (state, preset) => {
    _preset = preset;
    localStorage.setItem('preset', preset);
    _api.views.get(preset);
    _api.payload.get(preset);
    return { ...state, preset, view: { tabs: [] }, payload: {} };
  })
  .on(API.LOCK_UI, (state, lockedUI) => {
    return { ...state, lockedUI };
  })
  .on(API.HISTORY_UPDATE, (state, { undo, redo }) => {
    return { ...state, undo, redo };
  });

export default reducer;