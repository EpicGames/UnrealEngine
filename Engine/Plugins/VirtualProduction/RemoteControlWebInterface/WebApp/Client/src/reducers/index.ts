import { combineReducers } from 'redux';
import api, { ApiState } from './api';


export interface ReduxState {
  api: ApiState,
}

export default combineReducers({
  api,
});


export { _api } from './api';
