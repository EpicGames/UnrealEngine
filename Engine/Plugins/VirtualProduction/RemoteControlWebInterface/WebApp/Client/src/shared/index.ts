export enum PropertyType {
  Boolean =         'bool',
  Int8 =            'int8',
  Int16 =           'int16',
  Int32 =           'int32',
  Int64 =           'int64',
  Uint8 =           'uint8',
  Uint16 =          'uint16',
  Uint32 =          'uint32',
  Uint64 =          'uint64',
  Float =           'float',
  Double =          'double',
  Vector =          'FVector',
  Vector2D =        'FVector2D',
  Vector4 =         'FVector4',
  Rotator =         'FRotator',
  Color =           'FColor',
  LinearColor =     'FLinearColor',
  String =          'FString',
  Text =            'FText',

  Function =        'Function',
  StatelessController = 'FRCStatelessEventController',
}

export enum ConnectionSignal {
  Good =   'GOOD',
  Normal = 'NORMAL',
  Bad =    'BAD',
}

export interface ColorProperty {
  R: number;
  G: number;
  B: number;
  A?: number;
}

export interface VectorProperty {
  X: number;
  Y: number;
  Z: number;
  W?: number;
}

export interface RotatorProperty {
  Pitch: number;
  Yaw: number;
  Roll: number;
}

export type PropertyValue = boolean | number | string | VectorProperty | RotatorProperty | ColorProperty | IPayload;

export type JoystickValue = { [key: string]: number };

export type IHsvColor = {
  h: number;
  s: number;
  v: number;
  a?: number;
};

export interface IFunctionParameter {
  Name: string;
  Description: string;
  Type: PropertyType;
  Optional?: boolean;
  OutParameter?: boolean;
}

export interface IFunction {
  Name: string;
  Description?: string;
  Arguments: IFunctionParameter[];
}

export interface IExposedFunction {
  ID: string;
  DisplayName: string;
  UnderlyingFunction: IFunction;
  Metadata: { [key: string]: string };

  //Added
  Type: PropertyType;
}

export interface IProperty {
  Name: string;
  Description: string;
  Type: PropertyType;
  TypePath: string;
  Metadata: { [key: string]: string };
}

export interface IExposedProperty {
  ID: string;
  DisplayName: string;
  Metadata: Record<string, string>;
  Widget: WidgetType;
  UnderlyingProperty: IProperty;
  OwnerObjects: IObject[];

  //Added
  Type: PropertyType;
  TypePath: string;
}

export interface IController {
  ID: string;
  Name: string;
  Type: PropertyType;
  TypePath: string;
  Path: string;

  DisplayName: string;
  Metadata: Record<string, string>;
  Widget: WidgetType;


  UnderlyingProperty: IProperty;
  OwnerObjects: IObject[];
}

export interface IActor {
  Name: string;
  Path: string;
  Class: string;
}

export interface IExposedActor {
  DisplayName: string;
  UnderlyingActor: IActor;
}

export interface IGroup {
  Name: string;
  ExposedProperties: IExposedProperty[];
  ExposedFunctions: IExposedFunction[];
}

export interface IPreset {
  Path: string;
  Name: string;
  ID: string;
  Groups: IGroup[];
  
  ExposedProperties?: IExposedProperty[];
  ExposedFunctions?: IExposedFunction[];
  Controllers?: IController[];
  Exposed?: Record<string, IExposedProperty | IExposedFunction | IController>;
  IsFavorite?: boolean;
}

export type IPresets = { [preset: string]: IPreset };

export interface IObject {
  Name: string;
  Class: string;
  Path: string;
}

export interface IAsset extends IObject {
  Metadata: Record<string, string>;
}

export type IPayload = { [property: string]: PropertyValue | IPayload };

export type IPayloads = { [preset: string]: IPayload };

export enum WidgetTypes {
  Asset =           'Asset',
  Dial =            'Dial',
  Dials =           'Dials',
  Slider =          'Slider',
  Sliders =         'Sliders',
  ScaleSlider =     'Scale Slider',
  ColorPicker =     'Color Picker',
  MiniColorPicker = 'Mini Color Picker',
  ColorPickerList = 'Color Picker List',
  Toggle =          'Toggle',
  Joystick =        'Joystick',
  Button =          'Button',
  Text =            'Text',
  Label =           'Label',
  Dropdown =        'Dropdown',
  ImageSelector =   'Image Selector',
  Vector =          'Vector',
  Spacer =          'Spacer',
  Tabs =            'Tabs',
}

export type WidgetType = keyof typeof WidgetTypes | string;


export type IWidgetMeta = {
  Description?: string;
  Min?: number;
  Max?: number;
} & { [key: string]: any };


export enum IPanelType {
  Panel = 'PANEL',
  List  = 'LIST',
}

export interface IPanel {
  id?: string;
  title?: string;
  isTemplate?: boolean;
  type: IPanelType;
  widgets?: ICustomStackWidget[];
  items?: ICustomStackListItem[];
}

export interface IColorPickerList {
  id?: string;
  widget: WidgetTypes.ColorPickerList;
  items: ICustomStackProperty[];
}

export enum TabLayout {
  Stack =    'Stack',
  Screen =   'Screen',
  Empty =    'Empty',
}

export enum ScreenType {
  Stack =           'Stack',
  Playlist =        'Playlist',
  Snapshot =        'Snapshot',
  Sequencer =       'Sequencer',
  ColorCorrection = 'ColorCorrection',
  LightCards =      'LightCards',
}

export interface IScreen {
  type: ScreenType;
  data?: any;
}

export interface IDropdownOption {
  value: string;
  label?: string;
}

export interface ICustomStackProperty {
  id?: string;
  property: string;
  propertyType: PropertyType;
  widget: WidgetType;

  // Label only
  label?: string;

  // Vector only
  widgets?: WidgetType[];

  // Dropdown only
  options?: IDropdownOption[];

  // Function arguments
  args?: Record<string, any>;

  // Space only
  spaces?: number;
}

export interface ICustomStackItem {
  id?: string;
  label: string;
  widgets: ICustomStackWidget[];
}

export interface ICustomStackTabs {
  id?: string;
  widget: 'Tabs';
  tabs: ICustomStackItem[];
}

export interface ICustomStackListItem {
  id?: string;
  label: string;
  check?: { actor: string; property: string; };
  panels: IPanel[];
}

export type ICustomStackWidget = ICustomStackProperty | ICustomStackTabs | IColorPickerList;

export interface IView {
  tabs: ITab[];
}

export interface ITab {
  name: string;
  icon: string;
  layout: TabLayout;
  panels?: IPanel[];
  screen?: IScreen;
}

export interface IHistory {
  property: string;
  value: any;
  time: Date;
}