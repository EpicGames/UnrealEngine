.custom-ui-wrapper {
  width: 950px;
  display: flex;
  flex-direction: column;

  .search-wrapper {
    background-color: #171717;
    border-radius: 20px;
    height: 30px;
  }
  .group {
    flex: 1;
    display: flex;
    //height: 100%;

    &.vertical {
      flex-direction: column;
    }

    &.bottom-bar {
      flex: 0;
      height: 46px;
    }
  }

  .cc-group {
    background-color: #1F1F1F;
    flex: 1;
    margin: 5px;
    box-shadow: 0px 0px 5px 1px rgb(0 0 0 / 50%);
    border-radius: 4px;
    //height: 100%;
    //box-sizing: border-box;

    &.inline-group {
      display: flex;
      height: fit-content;
      padding: 0 10px;

      .group-item {
        display: flex;
        align-items: center;

        &.operation-style {
          min-width: 420px;
          flex: 0.3 1;
        }

        &.bottom-tabs-style {
          .tabs-elements {
            .tabs-header::after {
              bottom: 2px;
            }

            .tab {
              &.active {
                font-weight: bold;
              }
  
              &::before {
                display: block;
                content: attr(title);
                font-weight: bold;
                height: 0;
                overflow: hidden;
                visibility: hidden;
              }
            }

            .selector {
              border-color: #fff;
              top: 1px;
            }
          }
        }

        &.sensitivity-style {
          flex: 1;

          .tabs-container {
            width: 100%;
          }

          .tabs-elements {
            .tab {
              text-align: center;
              font-size: 17px;
            }
          }
        }
        
        .label {
          color: rgba(255, 255, 255, 0.35);
          margin-right: 1rem;
        }

        .tabs-container {
          padding: 0;
          height: auto;
        }
      }
    }

    &.disabled {
      &::after {
        content: '';
        top: 0;
        width: 100%;
        height: 100%;
        position: absolute;
        z-index: 10;
        background-color: rgba(27, 27, 27, .6);
      }
    }

    &.cc-panel {
      min-height: 463px;
    }

    .body {
      height: 100%;
    }

    .cc-title {
      background-color: #333333;
      padding: 5px 10px;
    }

    .control {
      display: flex;
      align-items: center;
      padding: 0px 10px;
      position: relative;
      margin-top: 10px;
      min-height: 42px;

      &.wrap {
        > label {
          position: absolute;
          top: -8px;
          width: inherit;
        }

        .slider-wrapper {
          margin: 0;
          width: 100%;
        }
      }

      &.disabled {
        pointer-events: none;
        opacity: 0.4;

        .circle {
          pointer-events: none;
        }
      }

      .slider-wrapper {
        margin-left: 15px;
      }

      label {
        width: 100px;
      }

      .input-field-container {
        input::-webkit-outer-spin-button,
        input::-webkit-inner-spin-button {
          -webkit-appearance: none;
          margin: 0;
        }
      }
    }

    .infinite-slider-container {
      width: 100%;
      display: flex;
      align-items: center;
    }
  }

  .shotbox {
    flex: 0.3;
    overflow: hidden;
    min-width: 565px;
    height: calc(100% - 10px);
    cursor: pointer;

    .trash-icon {
      font-size: 17px;
      margin: 0 20px;
    }
    
    .tabs-container {
      padding: 3px 2px;
    }

    .content {
      display: flex;
    }

    .body {
      height: fit-content;
    }

    .body {
      display: flex;
      flex-direction: row;
      flex-wrap: wrap;
      overflow: hidden;
      width: 100%;
    }
    .light-card {
      position: relative;
      display: flex;
      width: 99px;
      height: 99px;
      background-color: #121212;
      border: solid 1px #121212;
      border-radius: 4px;
      margin: 2px;
      justify-content: center;
      align-items: center;
      text-align: center;
      padding: 0px 5px;     
      overflow-wrap: anywhere;
      overflow: hidden;
      font-size: 14px;

      &.add-button {
        font-size: 28px;
        color: #9A9A9A;
      }

      &.active {
        border: solid 1px #5A91F7;
        color: #FFF;
      }

      &.disabled {
        opacity: 0.3;
      }

      .power-window-icon {
        position: absolute;
        top: 3px;
        left: 5px;
        transform: scale(0.5);
        transform-origin: top left;

        &.active {
          path {
            fill: #5A91F7;
            fill-opacity: 1;
          }
        }
      }

      .icon-favorite {
        top: 3px;
        right: 5px;
        position: absolute;
      }

      .change-indicators {
        position: absolute;
        bottom: 5px;
        width: 100%;
        display: flex;
        justify-content: space-evenly;
        font-size: 8px;
        color: #8E8E8E;

        .active {
          color: #3C89F7;
        }
      }
    }
  }

  .arrows-pagination {
    width: 63px;
    padding: 3px 5px 3px 0px;
    display: flex;
    flex-direction: column;
    justify-content: space-between;
    
    .arrow {
      cursor: pointer;
      background-color: #121212;
      height: calc(50% - 2px);
      display: flex;
      align-items: center;
      justify-content: center;

      &.disabled {
        opacity: 0.5;
      }

      &.up {
        .icon {
          &::before {
            border: none;
            background-color: #cdcccc;
            height: 50%;
            width: 30%;
            top: 50%;
            left: 35%;
          }

          &:after {
            left: 0;
            top: -50%;
            border-width: 20px 20px;
            border-bottom-color: #cdcccc;
          }
        }
      }

      &.down {
        .icon {
          &::before {
            border: none;
            background-color: #cdcccc;
            height: 50%;
            width: 30%;
            top: 0;
            left: 35%;
          }

          &::after {
            left: 0;
            top: 50%;
            border-width: 20px 20px;
            border-top-color: #cdcccc;
          }
        }
      }

      .icon {
        width: 40px;
        height: 40px;
        position: relative;
        display: inline-block;
        margin: 10px 10px;

        &::before,
        &::after {
          content: '';
          border-color: transparent;
          border-style: solid;
          position: absolute;
        }
      }
    }
  }

  .infinite-slider {
    width: 100%;
    margin-bottom: 10px;
    margin-right: 10px;

    
  }
}

.light-cards-wrapper {
  

  .color-picker-container {
    width: unset;
    height: unset;

    .color-picker-slider {
      display: none;
    }
  }
  .color-picker-wheel {
    flex-direction: row;
    width: 460px;
    height: 230px;
  }

  .slider-wrapper {
    width: 100%;
  }
}

.color-correction-wrapper {
  width: 100%;
  display: flex;
  flex-direction: column;
  max-height: 1000px;
  max-width: 1400px;
  box-sizing: border-box;

  &.LightCards {
    .grid-row {
      height: 40px;
    }

    .color-picker-singleton {
      height: 100%;
    }
  }

  .ref {
    position: absolute;
    top: 9px;
    opacity: 0.5;
    background-blend-mode: difference;
    pointer-events: none;
    width: 1400px;
    height: 1024px;
    z-index: 999;
    display: none;
  }

  .power-window-label {
    width: 100%;
    display: flex;
    justify-content: center;
    align-items: center;
    position: absolute;
    margin-top: 10px;
    user-select: none;
    color: #3c89f7;
    visibility: hidden;
    left: 0;
    z-index: 999;

    &.active {
      visibility: visible;
    }

   .wing {
      width: 200px;
      height: 20px;
      margin-left: 20px;
      background:  linear-gradient(90deg, #3c89f7, rgba(0, 0, 0, 0));
    }

    .wing.left {
      background:  linear-gradient(270deg, #3c89f7, rgba(0, 0, 0, 0));
      margin-right: 20px;
    }

  }
   
  .screen-mode-selectors {
    flex: 0;
    background-color: transparent;
    box-shadow: none;
  }

  .mode-button {
    width: 80px;
    display: flex;
    justify-content: center;
    align-items: center;
    height: 110px;
    margin-right: 0px;
    margin-left: 0px;
    cursor: pointer;
    border: solid 1px #1F1F1F;

    &.active {
      border-color: #5A91F7;
    }

    svg {
      width: 50px;
      height: 100%;

      &.ccr-icon {
        width: 25px;
      }
    }
  }

  .buttons-layout {
    height: calc(100% - 45px);
    display: flex;

    .buttons-content {
      flex: 1;
      display: flex;
      flex-direction: column;
      justify-content: space-around;

      &.small-button {
        .widget-button-wrapper {
          width: 110px;
          height: 50px;
        }
      }

      .space-buttons {
        display: flex;
        justify-content: space-between;
      }

      .widget-button-wrapper {
        height: 55px;
        margin: 0 auto;

        .widget-button {
          margin: 0;
          
          &:active {
            top: 0 !important; 
          }
        }
      }
    }
  }

  .grid-row {
    height: 95px;
    margin: 10px 0 0px 0px;

    &.orientation {
      height: 85px;
    }
  }

  .props-row {
    display: flex;
    justify-content: space-between;
    margin: 10px 10px;

    .prop-item {
      width: 50%;

      &:first-child {
        padding-right: 10px;
        border-right: 1px solid rgba(255, 255, 255, 0.1);
      }

      &:last-child {
        padding-left: 10px;
      }

      label {
        width: 90px;
      }

      .time-control {
        label { 
          width: 135px;
        }
      }
    }

    &.fit-dropdown {
      .prop .dropdown {
        max-width: 140px;
      }
    }

    .prop {
      display: flex;
      align-items: center;
      height: 40px;

      .widget-button-wrapper {
        height: 55px;
        
        .widget-button {
          margin: 15px 0 0;
          width: 100%;

          &:active {
            top: 0 !important; 
          }
        }
      }

      .dropdown {
        width: 100%;
        margin-left: 1rem;
        padding: 0 20px 0 10px;
        height: 25px;
        font-size: 15px;
        border-radius: 4px;
        border-color: #000;
        background-color: #191919;
        color: #FFF;
      }
    }
  }

  .vector-controls {
    display: flex;
    flex-direction: column;
    justify-content: space-around;

    .control {
      margin-top: 0 !important;
    }
  }

  .control-rows {
    height: calc(100% - 55px);
    overflow: hidden;
    display: flex;
    flex-direction: column;
    justify-content: space-evenly;
    color: #6D6D6D;
    
    &.light-advanced {
      justify-content: unset;

      .control.wrap {
        margin-top: 30px;
      }
    }

    &.no-space {
      .control {
        padding: 0;
      }
    }

    > .control {
      //margin-top: 1.5rem;
    }
  }

  .mode-selectors-wrapper {
    max-height: 40px;
  }
  
  .color-picker-wheel {



    .color-picker-controls {
      display: none;
    }

    .rgb-sliders {
      display: block;

      .slider-wheel-container {
        position: relative;
        height: 14px;
        margin: 28px 0px;
      }

      .slider-wheel-top {
        width: 280px;
      }

      .color-picker-slider-wheel {
        position: absolute;
        top: -0px;
        left: 20px;
        width: 225px;
        margin: 0px;
        min-height: 14px;

        .slider-circle {
          height: 6px;
          margin-right: 1px;
        }
      }
    }

    .rgb-inputs {
      display: none;
    }
  }

  @color-picker-size: 310px;

  .color-picker-singleton {
    display: flex;
    height: calc(100% - 40px);

    .color-picker-container {
      width: @color-picker-size;
      height: @color-picker-size;
      margin-top: 14px;
      margin-left: 29px;

      .color-picker-wheel {
        width: @color-picker-size;
        .circle {
          width: @color-picker-size - 20px;
          min-height: @color-picker-size - 20px;
          .circle-gradient {
            width: @color-picker-size - 20px;
            height: @color-picker-size - 20px;
            &:after {
              width: @color-picker-size - 20px;
              height: @color-picker-size - 20px;
            }
            .canvas-area {
              width: @color-picker-size - 30px;
              height: @color-picker-size - 30px;
            }
          }
        }

      }
  
      svg.fa-undo {
        display: none;
      }
    }
  
    .color-picker-slider {
      display: none;
    }
  
    
  
    .color-wheel {
      position: relative;
      margin-top: 20px;
      width: 365px;
      display: flex;
      flex-direction: column;
      justify-content: center;

      .reset-icon {
        position: absolute;
        left: 15px;
        top: 12px;
        cursor: pointer;
      }
  
      .wheel-addon {
        display: flex;
        flex-direction: row;
        align-items: center;
        justify-content: center;
  
        .dot {
          width: 25px;
          height: 25px;
          border-radius: 50%;
          background-color: #FFF;
          margin: 10px 5px;
        }
      }

      .color-picker-value {
        position: absolute;
        width: 292px;
        height: 292px;
        border-radius: 50%;
        z-index: 10;
        top: 23px;
        left: 38px;
        pointer-events: none;
        background-color: #000;

        @media only screen and (min-width: 1024px) {
          width: 192px;
          height: 192px;
          left: 38px;
          top: 23px;
        }
        
        @media only screen and (min-width: 1366px) {
          width: 312px;
          height: 312px;
          top: 23px;
          left: 38px;
        }
      }
    }
  
    .shift-slider {
      display: flex;
      justify-content: center;
      align-items: center;
      width: 30px;
      flex-direction: column;
      color: rgba(255, 255, 255, 0.35);
    }

    .color-picker-slider-wheel.vertical {
      overflow: initial;
      min-height: 280px;
      max-height: 210px;
      min-width: 20px;
      max-width: 20px;
      margin-top: -40px;

      &::after {
        content: '';
        position: absolute;
        display: block;
        width: calc(100% + 20px);
        height: calc(100% + 20px);
        z-index: 10;
        left: 50%;
        top: 50%;
        transform: translate(-50%, -50%);
      }

      &:before {
        background: transparent linear-gradient(0deg, #050505 0%, #23232300 19%, #1B1B1B00 81%, #080808 100%) 0% 0% no-repeat padding-box;
      }

      .circles-list {
        height: calc(100% - 25px);
        overflow: hidden;
        flex-direction: column;
        top: 15px;
        left: 7px;

        .slider-circle {
          margin-right: 0px;
          margin-bottom: 3px;
        }
      }
    }

    .alpha-slider.v-slider,
    .saturation-slider {
      .slider-circle {
        background: transparent;
        height: 24px;
        width: 10px;
        margin-right: 0px;
        box-sizing: border-box;
        border-color:#202020;
        border-style: solid;
        border-top-width: 6px;
        border-bottom-width: 6px;
        border-left-width: 2px;
        border-right-width: 2px;
      }
    }

    .details-section {
      flex: 1;
      margin: 10px 0px 0px 20px;
      max-width: calc(100% - 280px);

      .rgb-sliders {
        color: rgba(255,255,255,0.35);
        height: 100%;

        display: flex;
        flex-direction: column;
        justify-content: space-evenly;


        .input-field-container {
          .input-field {
            height: 22px;
            width: 70px;
            text-align: right;
          }

          input::-webkit-outer-spin-button,
          input::-webkit-inner-spin-button {
            -webkit-appearance: none;
            margin: 0;
          }
        }
        
        .slider-wheel-container {
          display: flex;
          width: 100%;
          justify-content: space-between;
          margin: 20px 0px;

          &:first-child {
            margin-top: 26px;
          }

          &.slider {
            height: 22px;
          }

          .title {
            width: 50px;
            display: flex;
            align-items: center;
            justify-content: right;
            margin-right: 10px;
          }

          .slider-wrapper {
            width: calc(100% - 130px);
          }

          .value {
            width: 55px;
            margin-left: 15px;
            display: flex;
            align-items: center;
          }
        }

        .color-picker-slider-wheel {
          margin: 0px;
          flex: 1;
          width: 100%;
          height: fit-content;
        }

        .hue-slider {
          .color-picker-slider-wheel {
            /*background: linear-gradient(90deg, rgb(255,  0,  0) 0%, 
                                               rgb(255,255,  0) 16%, 
                                               rgb(  0,255,  0) 33%, 
                                               rgb(  0,255,255) 50%, 
                                               rgb(  0,  0,255) 66%, 
                                               rgb(255,  0,255) 83%, 
                                               rgb(255,  0,  0) 100% );*/
          }
        }
      }

      .color-properties {
        display: flex;
        flex-direction: row;
        flex: 1;
        height: calc(100% - 20px);
        align-items: center;

        .sliders {
          width: 100%;
          height: 100%;
        }
      }

      .slider-mode-toggle {
        .labels {
          transform: rotate(90deg);
          transform-origin: 0 0;
          top: -1px;
          left: 32px;
          width: 100px;
          font-size: 11px;
        }
      }
    }
  }
  
  .mode-selectors {
    width: 100%;
    display: flex;
    
    .tabs-container {
      width: 100%;


      .tabs-header {
        .tab {
          font-weight: bold;
          color: #9C9C9C;
          position: relative;

          &.active {
            color: #FFF;
          }

          .change-indicator {
            position: absolute;
            top: 0px;
            right: -10px;
            font-size: 8px;
            visibility: hidden;
            color: #3C89F7;

            &.active {
              visibility: visible;
            }
          }

          &::before {
            display: block;
            content: attr(title);
            font-weight: bold;
            height: 0;
            overflow: hidden;
            visibility: hidden;
          }
        }

        .tab.active {
          font-weight: bold;
        }
      }
    }
    
    .mode-selector {
        display: flex;
        width: 100%;
        height: 30px;
        background-color: #1e1e1e;
        border: solid 1px #1e1e1e;
        border-radius: 5px;
        margin: 2px;
        justify-content: center;
        align-items: center;
        text-align: center;
        padding: 0px 5px;     
        overflow-wrap: anywhere;
        overflow: hidden;
        font-size: 14px;
        text-orientation: sideways;
        cursor: pointer;
        position: relative;
  
  
        &.active {
          border: solid 1px #5A91F7;
          background: #232323;
          color: #FFF;
          font-weight: bold;
        }

        .change-indicator {
          position: absolute;
          top: 5px;
          right: 5px;
          font-size: 8px; 
          visibility: hidden;
          color: #3C89F7;

          &.active {
            visibility: visible;
          }
        }

    }
  }
}

.color-property-selection {
  font-weight: 700;
}

.color-property-selection,
.sensitivity {
  display: flex;
  justify-content: space-evenly;
  color: rgba(156, 156, 156, 0.75);
  margin: 7px 15px 0px 40px;
  min-height: 16px;
  max-width: 800px;
  font-size: 18px;

    .button {
      cursor: pointer;
    }
    
    .active {
      color: #FFF;
    }
    
    .mode-selector {
      cursor: pointer;
      }
  }

.slider-wheel-container,
.infinite-slider {
  &.red-slider {
    .slider-circle {
      background-color: red;
      border-color: red;
      border-radius: 0px;
      height: 12px;
      width: 4px;
    }
    .slider {
      background:  linear-gradient(90deg, rgba(255,255,255,1) 0%, rgba(255,0,0,1) 100%);
    }
    .range-fill {
      background-color: transparent;
    }
  }
  &.green-slider {
    .slider-circle {
      background-color: #0f720f;
      border-color: #0f720f;
      border-radius: 0px;
      height: 12px;
      width: 4px;
    }
    .slider {
      background:  linear-gradient(90deg, rgba(255,255,255,1) 0%, rgba(0,255,0,1) 100%);
    }
    .range-fill {
      background-color: transparent;
    }
  }
  &.blue-slider {
    .slider-circle {
      background-color: blue;
      border-color: blue;
      border-radius: 0px;
      height: 12px;
      width: 4px;
    }
    .slider {
      background:  linear-gradient(90deg, rgba(255,255,255,1) 0%, rgba(0,0,255,1) 100%);
    }
    .range-fill {
      background-color: transparent;
    }
  }
  &.alpha-slider {
    .slider-circle {
      background-color: #383838;
      border-color: #383838;
      border-radius: 0px;
      height: 12px;
      width: 4px;
    }
    .slider {
      background:  linear-gradient(90deg, rgb(71, 71, 71) 0%, rgb(255, 255, 255) 100%);
    }
    .range-fill {
      background-color: transparent;
    }
  }

  &.saturation-slider {
    .slider {
      background: -webkit-linear-gradient(right, rgb(255 255 255 / 0%) 0%, #474747 100%), -webkit-linear-gradient(left, white 0%, rgba(255, 255, 255, 0) 100%);
      background-color: #fff;
    }

    .range-fill {
      background-color: transparent;
    }

    .color-picker-slider-wheel {
      background: -webkit-linear-gradient(right, rgba(230, 230, 230, 0) 0%, black 100%), -webkit-linear-gradient(left, white 0%, rgba(255, 255, 255, 0) 100%);

      &::before {
        border-radius: 0px;
      }
    }
  }
}

.Temperature-slider {
  .slider {
    background: linear-gradient(90deg, rgba(120,194,255,1) 0%, rgba(255,92,0,1) 100%);
  }
  .slider-wrapper .slider-block .range-fill {
    background-color: transparent;
  }
}

.DistanceFromCenter-slider,
.Longitude-slider,
.Latitude-slider,
.GuideScale-slider,
.Spin-slider,
.Exposure-Compensation-slider,
.Intensity-slider,
.color-grading-ShadowsMax-slider,
.color-grading-HighlightsMin-slider,
.Falloff-slider,
.Inner-slider,
.ColorGradingSettings-ShadowsMax-slider,
.ColorGradingSettings-HighlightsMin-slider,
.Outer-slider,
.Priority-slider,
.Opacity-slider,
.Feathering-slider,
.Scale-X-slider,
.Scale-Y-slider,
.WallIndex-slider,
.Unit-slider,
.Tint-slider {
  .slider {
    background: linear-gradient(90deg, rgba(57,57,57,1) 0%, rgba(255,255,255,1) 100%);
  }
  .slider-wrapper .slider-block .range-fill {
    background-color: transparent;
  }
}

.GuideColor-R-slider {
  .slider {
    background:  linear-gradient(90deg, rgba(255,255,255,1) 0%, rgba(255,0,0,1) 100%);
  }
  .slider-wrapper .slider-block .range-fill {
    background-color: transparent;
  }
}
.GuideColor-G-slider {
  .slider {
    background:  linear-gradient(90deg, rgba(255,255,255,1) 0%, rgba(0,255,0,1) 100%);
  }
  .slider-wrapper .slider-block .range-fill {
    background-color: transparent;
  }
}
.GuideColor-B-slider {
  .slider {
    background:  linear-gradient(90deg, rgba(255,255,255,1) 0%, rgba(0,0,255,1) 100%);
  }
  .slider-wrapper .slider-block .range-fill {
    background-color: transparent;
  }
}



.asset-control {
  .reset {
    padding: 10px;
    cursor: pointer;
  }

  .asset-item {
    padding: 0;
    margin: 0;
    margin-left: 10px;

    .label {
      min-width: 50px;
      margin: 0;
    }

    .image {
      margin: 0;
    }

    .input-title {
      margin-left: 10px;
      background-color: #121212;
      border-radius: 3px;

      input {
        padding: 0;
        height: 38px;
        padding-left: 5px;
        font-size: 11px;
        text-overflow: ellipsis;
      }
    }

    .asset-icon {
      margin-right: 10px;
    }

    img.box-image {
      width: 35px;
      height: 35px;
    }
  }
}

.multilines-content {
  height: calc(100% - 60px);
  display: flex; 

  &.full-height {
    height: calc(100% - 5px);
  }

  .content .body {
    width: 100%;
    display: flex;
    flex-direction: column;
    justify-content: space-evenly;

    .control.wrap > label {
      top: -8px;
    }
  }
}

.guides-color-picker {
  flex: 1;
  padding: 0 10px;

  .control-rows > .control {
    margin-top: 1rem;
  }

  .label {
    margin-bottom: .5rem;
  }

  .color-picker-container {
    width: 150px;
    height: 150px;

    .color-picker-wheel {
      padding: 0;
    }

    .circle {
      margin-top: 0;
      background-color: transparent;
    }

    .circle-gradient {
      width: 145px;
      height: 143px;
      top: 80px;
    }

    .color-picker-wheel,
    .circle,
    .circle-gradient::after {
      width: 100% !important;
      height: 100% !important;
      min-height: 100% !important;
    }

    .canvas-area {
      width: calc(100% - 10px) !important;
      height: calc(100% - 10px) !important;
    }
  }

  .color-controls-row {
    display: flex;

    .control-rows {
      flex: 1;

      .control label {
        width: 25px;
      }
    }
  }
}