@import './playlist.less';
@import './snapshot.less';
@import './sequencer.less';
@import './colorCorrections.less';


.droppable .draggable-item.tab-group {
  margin: 10px 15px;
}

.tab-group {
  border: solid 2px @widget-tab-content-background;
  border-radius: 5px;
}

.tab-widget-wrapper {
  display: flex;
  flex-direction: row;
  justify-content: space-between;
  margin: 0px;
  padding: 0px;

  .btn {
    width: 100%;
    height: 30px;
    display: flex;
    align-items: center;
    justify-content: center;
    color: #fff;
    background-color: @widget-tab-content-background;
    text-transform: uppercase;
    margin: 0px;
    border-radius: 0px;
    border: 0px;

    border-top: solid 0px  @tabs-content-background-color;
    border-right: solid 1px @panel-background-color;
    
    &:last-child {
      border-right: 0px;
    }

    &.selected {
      background-color: @panel-background-color;
      color: @tabs-selected-text-color;

      &:hover {
        background-color: inherit;
      }
    }
  }

}

.slider-row {
  box-sizing: border-box;
  position: relative;
  display: flex;
  flex-direction: row;
  align-items: center;
  width: calc(100% - 18px);
  min-height: 40px;
  padding: 0px 35px 0 0;
  
  .title {
    width: @actor-title-width;
    min-width: @actor-title-width;
    margin: 10px;
  }

  .limits {
    width: 70px;
    margin: 5px;
    text-align: center;
  }

  .color {
    width: 100%;
    height: 30px;
    margin: 7px 7px 7px -10px;
    border-radius: 4px;
  }

  svg.expand-icon {
    margin-right: 1rem;
  }

  svg[data-icon="undo"]:not(.expand-icon) {
    position: absolute;
    right: 0;
  }

  svg:not(.drag-icon) {
    padding: 5px;
  }
}

.text-row {
  .widget-text-wrapper {
    padding: 0;

    .widget-text {
      height: 30px;
    }
  }
}

.actor-color {
  margin: 10px;
  display: flex;

  .color {
    min-width: 30px;
    height: 24px;
    margin: 0px 0px 0px 10px;
    border-radius: 4px;
    border: solid 2px #FFF;
  }

  svg {
    margin: auto;
  }

  .selected {
    color: @text-color;
    background-color: @selected-background-color;
  }

  .btn-title span {
    max-height: 30px;
    overflow: hidden;
  }

  .btn-title,
  .btn {
    width: 170px;
    height: 45px;
    margin: 0 10px;
    padding: 10px 8px 10px 10px;
    display: flex;
    justify-content: space-between;
    align-items: center;
    border: solid 1px #000;
    background-color: @background-button-color;
    color: inherit;

    span {
      flex: 1;
    }

    p {
      max-height: 30px;
      overflow: hidden;
      text-align: start;
    }
  }

  button {
    outline: none;
  }

}

.multiline-widget-wrapper {
  display: flex;
  flex-wrap: wrap;

  .btn-wrapper {
    width: 200px;
    height: 48px;
    margin: 5x;
    border: 1px solid transparent;

    .widget-button-wrapper {
      width: 100%;
      overflow: initial;

      .widget-button {
        height: 40px;
        top: 2px;
        left: 50%;
        transform: translateX(-50%);
        margin: 0;
        line-height: 18px;

        overflow: hidden;
        display: -webkit-box;
        -webkit-line-clamp: 1;
        -webkit-box-orient: vertical; 

        &:active {
          top: 4px !important;
        }
      }
    }
  }
}

.slider-row,
.toggle-row,
.dropdown-row {
  .title {
    width: 130px;
    min-width: 130px;
    margin: 0 10px;
    max-height: 34px;
    overflow: hidden;
    text-align: start;
    font-size: 14px;
    line-height: 16px;
  }
}
