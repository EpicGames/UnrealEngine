.playlist-wrapper {
  display: flex;
  flex-direction: column;
  font-size: 12px;
  line-height: 20px;
  outline: none;
  background-color: @panel-background-color;
  margin: 20px;
  height: calc(100% - 80px);
  max-width: 980px;
  width: calc(100% - 80px);

  .playlist-placeholder {
    height: 100%;
    display: flex;
    align-items: center;
    justify-content: center;
    font-size: 35px;

    &-title {
      cursor: pointer;
      display: flex;
      align-items: center;

      .folder-icon {
        margin-right: 1rem;
      }
    }
  }

  .btn.playlist-play {
    &:hover {
      color: @playlist-active-play-color;
    }
    &.active {
      color: @playlist-active-play-color;
    }
  }

  .btn.playlist-stop {
    &:hover {
      color: @playlist-active-stop-color;
    }
    &.active {
      color: @playlist-active-stop-color;
    }
  }

  .btn.playlist-rehold {
    &:hover {
      color: @playlist-active-rehold-color;
    }
    &.active {
      color: @playlist-active-rehold-color;
    }
  }

  .playlist-item {
    box-shadow: 0 5px 0px 3px @playlist-border-color;
  }

  .icon {
    cursor: pointer;
    padding: 3px;
  }

  .playlist-header {
    display: flex;
    align-items: center;
    margin-bottom: 0.3rem;
    cursor: pointer;

    h1 {
      margin: 0;
    }

    .icon {
      margin-left: 1rem;
      margin-right: 1rem;
      font-size: 18px;
    }
  }

  .playlist-controls {
    display: flex;
    justify-content: space-between;
    align-items: center;
    margin: 10px 10px 0px 10px;

    .controls {
      .btn {
        border-radius: 0px;
        margin-left: 0px;
        margin-right: 0px;
        width: 60px;
        height: 55px;
        font-size: 20px;

        &:first-child {
          border-top-left-radius: 4px;
          border-bottom-left-radius: 4px;
          border-right: 0px;
        }
        &:last-child {
          border-top-right-radius: 4px;
          border-bottom-right-radius: 4px;
          border-left: 0px;
        }
      }
    }

    .description {
      margin-left: 1rem;
    }

    h1 {
      flex: 1;
    }

    .btn-round {
      border-radius: 50%;
      width: 65px;
      height: 65px;
      font-size: 20px;
    }
  }
  
  .list-wrapper {
    height: calc(100% - 75px);
    border-bottom: 0;
  }

  .search-block {
    width: 100%;
    height: 35px;
    display: flex;
    align-content: center;
    padding-top: 15px;
    margin-bottom: 5px;
    margin-left: 0.7rem;

    .add-item {
      white-space: nowrap;
    }

    .search-field {
      width: 100%;
      height: 23px;
      display: flex;
      align-items: center;
      margin: 0 50px;
      padding: 5px 10px;
      border-radius: 5px;
      border: @primary-border;
  
      svg {
        width: 15px;
        height: 15px;
        margin: 0 15px 0 0;
        color: @secondary-selected-color;
        cursor: pointer;
      }
  
      input {
        width: 100%;
        height: 20px;
        background-color: initial;
        border: none;
      }
    }

    .snapshot-btn {
      width: 25%;
      height: 35px;
      padding: 0;
      display: flex;
      align-items: center;
      justify-content: center;
      background-color: @secondary-selected-color;
      color: #FFF;
    }
  }

  .table-wrapper {
    overflow: auto;
    height: calc(100% - 55px);
    
    .table-container {
      height: calc(100vh - 275px);
      overflow: auto;
    }

    .controls {
      margin-top: 10px;
      margin-bottom: 10px;

      button.btn {
        border-radius: 0px;
        margin: 0;
        padding: 8px 12px;
        width: 60px;

        

        &:first-child {
          border-top-left-radius: 4px;
          border-top-right-radius: 4px;
          border-bottom: 0px;
        }
        &:last-child {
          border-bottom-left-radius: 4px;
          border-bottom-right-radius: 4px;
          border-top: 0px;
        }
      }
    }

    th {
      font-size: 14px;
      background-color: #2f2f2f;
      padding: 5px;
      font-weight: normal;
      top: 2px;
      z-index: 10;
      position: sticky;

      &::before {
        content: '';
        position: absolute;
        width: 100%;
        height: 5px;
        background-color: #2f2f2f;
        top: -4px;
        left: 0;
      }
    }

    tr {
      td {
        padding: 0 15px;
        cursor: default;

        &:not(:last-child) {
          > div {
            margin-top: 1rem;
          }
        }
  
        &.col-actions {
          padding: 0 5px;
        }
      }
    }

    .col-name {
      width: 600px;

      .icon {
        display: block;
        margin-top: .5rem;
      }
    }

    .col-control {
      width: 65px;
      padding: 0px 5px;

      .checkbox-container {
        margin: 0 auto;
        width: 30px;

        .checkmark {
          display: flex;
          justify-content: center;
          align-items: center;
          border: 0px;
          background-color: transparent;

          .playlist-icon {
            z-index: 2;
          }
        }
        
        /* On mouse-over, add a grey background color */
        .container:hover input ~ .checkmark {
          background-color: @input-field-background-color;
        }
        
        /* When the checkbox is checked, add a blue background */
        .container input:checked ~ .checkmark {
          background-color: @input-field-background-color;
        }
        
        /* Create the checkmark/indicator (hidden when not checked) */
        .checkmark:after {
          content: "";
          position: absolute;
          display: none;
        }
        
        /* Show the checkmark when checked */
        .container input:checked ~ .checkmark:after {
          display: block;
        }
        
        /* checkmark/indicator */
        .container .checkmark:after {
          width: calc(100% - 5px);
          height: calc(100% - 3px);
          background-color: @widget-dial-needle-color;
          transform: none;
          left: 0px;
          top: 0px;
          border-radius: 4px;
        }

      }

      .input-field {
        width: 60px;
        padding: 0;
      }

      .input-container:not(:last-child) {
        margin-bottom: 0.5rem;
      }
    }
    
    .col-4 {
      width: 50px;
    }

    table { 
      width: 100%;
    }
  }

  select {
    background-color: @widget-dropdown-background;
    border-color: @widget-dropdown-border;
    color: @widget-dropdown-color;
    padding: 5px;
    width: 100%;
  }

  .tooltip-icon {
    position: relative;
    border-radius: 5px;

    &:hover {
      background-color: #2f2f2f;
    }
  }

  .playlist-tooltip {
    position: absolute;
    background-color: #2f2f2f;
    cursor: default;
    width: 330px;
    left: -360px;
    top: 0;
    height: 170px;
    z-index: 1;
    padding: 15px;
    border-radius: 5px;

    &-header {
      font-size: 13px;
      opacity: 0.3;
      text-transform: uppercase;
      position: relative;

      &::after {
        content: '';
        position: absolute;
        width: calc(100% - 105px);
        height: 2px;
        background-color: #fff;
        opacity: 0.3;
        top: 6px;
        right: 0px;
      }
    }
  }
}

img.box-image {
  width: 60px;
  height: 60px;
  box-shadow: 0px 0px 3px 2px rgba(0,0,0,0.7);
  border-radius: 3px;
  border-bottom: solid 2px @sequencer-color;
}

.playlist-icon {
  &.large {
    width: 40px;
    margin: 0px 10px;
  }
}