.sequencer-wrapper {
  .list-wrapper {
    height: 100%;
    border-bottom: 0;
  }

  .search-block {
    .search-field {
      width: 100%;
    }
  }

  .table-wrapper {
    max-height: calc(100% - 70px);
    td {
      padding: 0 15px;
      cursor: default;
    }

    .col {
      flex-grow: 1;
    }

    .name-block {
      img {
        box-shadow: 0px 0px 3px 2px rgba(0,0,0,0.3);
        border-radius: 3px;
        border-bottom: solid 2px @sequencer-color;
        margin: 5px 10px 5px 5px;
      }
    }
    
    .col-4 {
      width: 50px;
    }
  }
}