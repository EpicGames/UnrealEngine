@save-title-width: 100px;
@btn-inner-height: 30px;

.snapshot-wrapper {  
  display: flex;
  flex-direction: column;
  font-size: 12px;
  line-height: 20px;
  outline: none;
  background-color: @panel-background-color;
  margin: 20px;
  height: calc(100% - 45px);
  max-width: 980px;
  width: calc(100% - 80px);

  .list-wrapper {
    height: calc(100% - 210px);
    border-bottom: solid 1px #000;
    padding: 20px;
  }
  .search-block {
    width: 100%;
    height: 35x;
    display: flex;
    align-content: center;

    .snapshot-btn {
      width: 25%;
      height: 35px;
      padding: 0;
      display: flex;
      align-items: center;
      justify-content: center;
      background-color: @secondary-selected-color;
      color: #FFF;
    }
  }

  .search-field {
    width: calc(70% - 100px);
    height: 23px;
    display: flex;
    align-items: center;
    margin: 0 50px;
    padding: 5px 10px;
    border-radius: 5px;
    border: @primary-border;
    cursor: pointer;

    svg {
      width: 15px;
      height: 15px;
      margin: 0 15px 0 0;
      color: @secondary-selected-color;
      cursor: pointer;
    }

    input {
      width: 100%;
      height: 20px;
      background-color: initial;
      border: none;
      cursor: pointer;
    }
  }

  table, th, td {
    border-collapse: collapse;
  }

  tr {
    height: 45px;
    border-top: 1px solid rgb(83, 82, 82);
    cursor: pointer;
  }

  tr:first-child {
    border-top: none;
  }

  tr:last-child {
    border-bottom: 1px solid rgb(134, 134, 134);
  }

  .selected-row {
    background-color: #335;
  }

  td {
    padding: 10px;
    font-size: 0.8em;
    line-height: 18px;
    overflow: hidden;
    text-align: start;

    .name-block {
      display: flex;
      align-items: center;
    }

    img {
      max-height: 38px;
      max-width: 38px;
      width: 100%;
      height: 100%;
      object-fit: contain;
      margin: 0 10px 0 5px;
    }

    p {
      max-height: 35px;
      word-break: break-all;
      overflow: hidden;
      text-align: start;
    }

    .play-icon {
      display: flex;
      justify-content: center;
      border: 1px solid #FFF;
      padding: 5px 0;
      transition: background-color .3s;
      cursor: pointer;

      svg {
        transition: color .3s;
      }

      &:hover {
        background-color: #fff;
        color: #202020;
      }
    }
  }

  th {
    height: 40px;
    text-align: start;
    padding: 0 15px;
    position: sticky;
    top: -2px;
    background: #202020;
  }

  thead {
    .name {
      padding: 0 0 0 55px;
    }

    .created-header {
      display: flex;
      justify-content: space-between;
      align-items: center;

      svg {
        width: 15px;
        height: 15px;
        color: @secondary-selected-color;
      }
    }

    tr {
      cursor: initial;
    }
  }

  .table-wrapper {
    width: 100%;
    max-height: calc(100% - 20px);
    overflow-y: auto;
  }

  table {
    width: calc(100% - 5px);
    table-layout: fixed;
    margin: 10px 0 0 0;

    .name {
      width: 40%;
      padding: 0 0 0 50px;
      cursor: pointer;

      .name-block {
        width: 100%;
        display: flex;
        flex-direction: row;
        align-items: center;
      }

      .box {
        max-height: 38px;
        max-width: 38px;
        height: 100%;
        width: 100%;
        object-fit: contain;
        margin: 0 10px 0 5px;
      }
    }

    .description {
      width: 30%;
      cursor: pointer;
    }

    .created {
      width: 30%;
      cursor: pointer;
    }

    .sort-icon {
      margin-left: .5rem;
    }
  }

  .controls-block {
    width: 100%;
    display: flex;
    flex-direction: row;
    align-items: center;
    justify-content: flex-start;
    padding: 0px 20px;
    position: absolute;
    bottom: 10px;
    left: 0px;
  }

  .snapshot-btn {
    width: 30%;
    height: 40px;
    display: flex;
    justify-content: center;
    align-items: center;
    text-transform: uppercase;
    font-weight: bolder;
    margin: 0 30px;
    cursor: pointer;
    background-color: @color-primary-color;
    color: #FFF;

    p {
      max-height: @btn-inner-height;
      overflow: hidden;
    }
  }
  
  .snapshot-preview-wrapper {
    height: 25%;
    padding: 20px;
    padding-top: 0px;
    position: relative;

    .hint-text {
      color: rgb(75, 75, 75);
    }
    .preview-title {
      display: flex;
      justify-content: space-between;
      font-size: 25px;
      margin: 20px 0 0 0;

      svg {
        width: 20px;
        height: 20px;
        cursor: pointer;
      }

    }

    .description-title {
      width: 100%;
      font-size: 16px;
      font-weight: bolder;
      margin: 10px 0;
    }

    .preview-image {
      width: 30vw;
      height: 20vw;
      margin: 30px auto;
      background-color: rgb(238, 52, 191);
    }

    .preview-description {
      width: 100%;
      margin: 30px 0;
    }

    .controls-block {
      justify-content: space-between;

      .snapshot-btn {
        width: 50%;
      }

      .snapshot-dropdown {
        width: 40%;
        display: flex;
        flex-direction: row;
        align-items: center;
        justify-content: center;
  
        .title {
          margin: 0 10px 0 0;
        }
  
        .dropdown {
          width: 80%;
          height: 30px;
          color: @text-color;
          background-color: @background-button-color;
          border-radius: 5px;
          outline: none;
          padding: 0 10px;
          cursor: pointer;
        }
      }
    }
  }
}

.save-asset-modal {
  width: 500px;
  height: 290px;
  background-color: @color-navigation-background;
  border: solid 1px rgb(70, 70, 70);
  border-radius: 5px;
  margin: auto;
  padding: 15px;

  .save-row {
    width: 100%;
    height: 30px;
    display: flex;
    justify-content: space-between;

    svg {
      cursor: pointer;
    }
  }

  .item-title {
    width: @save-title-width;
    text-align: end;
    padding: 10px;
  }

  .item-content {
    width: calc(100% - @save-title-width);
    border-radius: 5px;
    background-color: @color-primary-background;
    border: none;
    outline: none;
    color: #FFFFFF;
    padding: 10px;
    cursor: text;
  }

  .item-row {
    width: 100%;
    display: flex;
    justify-content: flex-start;
    margin: 10px 0;
  }

  .path {
    padding: 10px;
    background-color: inherit;
  }

  .snapshot-btn {
    width: 50%;
    height: 40px;
    text-transform: uppercase;
    font-weight: bolder;
    margin: 10px 0 0 50%;
    background-color: @secondary-selected-color;
    cursor: pointer;
  }
}

.snapshot-btn {
  position: relative;

  .spin-icon {
    position: absolute;
    right: 20px;
  }
}