@keyframes loadingIcon {
  0% {opacity: 1; }
  50% { opacity: 0.5; }
  100% {opacity: 1; }
}