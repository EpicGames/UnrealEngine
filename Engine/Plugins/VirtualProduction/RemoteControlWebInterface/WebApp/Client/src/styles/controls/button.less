.widget-button-wrapper {
  height: 0px;
  width: 100%;
  overflow: hidden;
  padding-top: 591.44px / 1127.34px * 100%;
  position: relative;
}

.widget-button {
  position: absolute;
  top: 0;
  left: 0;
  display: flex;
  align-items: center;
  justify-content: center;
  width: calc(100% - 20px);
  height: calc(100% - 20px);
  margin: 10px;
  border: 1px solid #3d3f48;
  border-radius: 5px;
  box-shadow: @widget-button-shadow;
  background: @widget-button-background;
  font-family: 'Oswald', Helvetica;
  line-height: 43px;
  color: @text-color;
  text-decoration: none;
  text-shadow: -1px -1px 0 #3d3f48;
  transition: initial;
  cursor: pointer;

  &:focus {
    outline: none;
  }
  &:active {
    top: -16px !important;
    box-shadow: 0 4px 9px 2px rgba(22, 22, 22, .5) inset, 0 -1px 0 rgba(255, 255, 255, .1) inset;
    color: #CECECE;
  }
}