/* Customize the label (the container) */
.container {
  display: block;
  position: relative;
  padding-left: 35px;
  cursor: pointer;
  font-size: 22px;
  -webkit-user-select: none;
  -moz-user-select: none;
  -ms-user-select: none;
  user-select: none;
}

/* Hide the browser's default checkbox */
.container input {
  position: absolute;
  opacity: 0;
  cursor: pointer;
  height: 0;
  width: 0;
}

.checkmark {
  position: absolute;
  top: -12px;
  left: 5px;
  height: 23px;
  width: 23px;
  background-color: @input-field-background-color;
  border: solid 1px;
  border-color: @input-field-border-color;
  border-radius: 4px;
}

/* On mouse-over, add a grey background color */
.container:hover input ~ .checkmark {
  background-color: @input-field-background-color;
}

/* When the checkbox is checked, add a blue background */
.container input:checked ~ .checkmark {
  background-color: @input-field-background-color;
}

/* Create the checkmark/indicator (hidden when not checked) */
.checkmark:after {
  content: "";
  position: absolute;
  display: none;
}

/* Show the checkmark when checked */
.container input:checked ~ .checkmark:after {
  display: block;
}

/* checkmark/indicator */
.container .checkmark:after {
  left: 7px;
  top: 3px;
  width: 5px;
  height: 10px;
  border: solid @widget-dial-needle-color;
  border-width: 0 3px 3px 0;
  -webkit-transform: rotate(45deg);
  -ms-transform: rotate(45deg);
  transform: rotate(45deg);
}