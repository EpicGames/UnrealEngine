.tab-group {
  .color-picker-container {
    width: 283px;
  }
}

.color-picker-container {
  .fa-undo,
  .expand-icon {
    margin-top: 10px;
  }
}

.color-picker {

  &-container {
    height: 100%;
    display: flex;
    width: 295px;
    min-height: 115px;
    justify-content: center;
  }

  &-sliders {
    flex: 1;

    .slider-row {
      position: relative;
      min-height: 20px;
      width: 100%;
      border-radius: 12px;
      border: solid 2px #1E1C1E;
      margin-bottom: 1rem;      

      &.red {
        background-image: linear-gradient(to right, #000000, #F00);
      }

      &.green {
        background-image: linear-gradient(to right, #000000, #0F0);
      }

      &.blue {
        background-image: linear-gradient(to right, #000000, #00F);
      }

      &-pointer {
        position: absolute;
        width: 15px;
        height: 15px;
        border-radius: 50%;
        border: 2px solid #423e49;
      }
    }
  }

  &-wheel {
    width: 250px;
    height: calc(100% - 10px);
    padding-bottom: 10px;
    display: flex;
    flex-direction: column;
    align-items: center;

    .rgb-inputs {
      display: flex;
      align-items: center;
      margin-top: 10px;
    }
    
    .rgb-sliders {
      display: none;
    }

    .rectangle {
      position: relative;
      margin-top: 10px;
      width: 180px;
      height: 180px;
      border-radius: 12px;
      touch-action: none;
      cursor: pointer;
      background: -webkit-linear-gradient(top, rgba(230, 230, 230, 0) 0%, black 100%), -webkit-linear-gradient(left, white 0%, rgba(255, 255, 255, 0) 100%);;

      &-area-pointer {
        position: absolute;
        width: 15px;
        height: 15px;
        background: radial-gradient(circle, rgba(255,255,255,0) 0%, rgba(255,255,255,0) 49%, rgba(76,76,76,1) 50%, rgba(114,114,114,1) 60%, rgba(76,76,76,1) 70%);
        border: 2px solid #5d5d5d;
        top: 0;
        left: 0;
        z-index: 10;
        border-radius: 50%;
        cursor: pointer;
      }
    }

    .circle {
      position: relative;
      margin-top: 10px;
      width: 210px;
      min-height: 210px;
      border-radius: 50%;
      background-color: #1E1C1E;
      cursor: pointer;

      .circle-gradient {
        position: absolute;
        display: flex;
        align-items: center;
        justify-content: center;
        top: 50%;
        left: 50%;
        transform: translate(-50%, -50%);
        width: 210px;
        height: 210px;
        z-index: 5;
        background: conic-gradient(from -270deg, #ff0023, #ffed02, #00ff07, #00e0ff, #1402ff, #ff00f2, #ff0326);
        border-radius: 50%;

        &::after {
          content: '';
          position: absolute;
          top: 0;
          left: 0;
          width: 210px;
          height: 210px;
          background: radial-gradient(circle closest-side, rgba(255, 255, 255, 1), transparent);
          border-radius: 50%;
          pointer-events: none;
        }

        .canvas-area {
          position: relative;
          width: 200px;
          height: 200px;
          background-color: rgba(30, 28, 30, .3);
          border-radius: 50%;
          touch-action: none;

          &:after,
          &::before {
            content: '';
            position: absolute;
            z-index: 1;
            background-color: rgba(255,255,255,0.5);
            left: 50%;
            top: 50%;
            transform: translate(-50%, -50%);
          }

          &::after {
            width: calc(100% - 1px);
            height: 1px;
          }

          &::before {
            width: 1px;
            height: calc(100% - 1px);
          }

          &-pointer {
            position: absolute;
            width: 15px;
            height: 15px;
            background: rgba(207,207,207,0.5);
            border: 2px solid #FFF;
            top: calc(50% - 10px);
            left: calc(50% - 9px);
            z-index: 10;
            border-radius: 50%;
            cursor: pointer;
          }
        }
      }
    }

    .color-picker-inputs {
      display: flex;
      align-items: center;
      margin-top: 10px;

      input[type=number]::-webkit-outer-spin-button,
      input[type=number]::-webkit-inner-spin-button {
        -webkit-appearance: none;
        margin: 0;
      }

      .input {
        position: relative;

        &:not(:last-child) {
          margin-right: 1rem;
        }

        &::after {
          content: '';
          display: block;
          bottom: -5px;
          position: absolute;
          height: 2px;
          width: 100%;
        }

        &.red {
          &::after {
            background-color: #dd352f;
          }
        }

        &.green {
          &::after {
            background-color: #169b32;
          }
        }

        &.blue {
          &::after {
            background-color: #2262bc;
          }
        }

        &-field {
          width: 60px;
          height: 30px;
          border-radius: 0;
          border: 1px solid #1a1a1a;
          cursor: unset;
        }
      }

      .color {
        width: 15px;
        height: 15px;
        border-radius: 50%;
      }
    }

    

    .color-picker-slider {
      touch-action: none;
      position: relative;
      border-radius: 10px;
      margin: 20px 0;
      padding: 5px 0; 
      color: #8a8a8a;
      font-weight: 500;
      font-size: 15px;

      &::after,
      &::before {
        position: absolute;
        content: '';
        width: 2px;
        height: 10px;
        background-color: #3d3d3d;
        bottom: -12px;
        border-radius: 10px;
      }

      &::before {
        left: 1px;
      }

      &::after {
        right: 1px;
      }

      &-label {
        position: absolute;
        top: -25px;
        left: -10px;
      }

      &-value {
        position: absolute;
        right: -10px;
        top: -25px;
      }

      &-bgc {
        height: 5px;
        background-color: #000;
        cursor: pointer;
        position: relative;
      }

      &-pointer {
        position: absolute;
        z-index: 1;
        width: 18px;
        height: 18px;
        border-radius: 18px;
        background-color: #848484;
        top: -2px;
        left: -7px;
        cursor: pointer;

        &::after,
        &::before {
          content: '';
          width: 6px;
          height: 12px;
          position: absolute;
          top: 3px;
        }

        &::before {
          border-bottom-left-radius: 9px;
          border-top-left-radius: 9px;
          background-color: #242424;
          left: 3px;
        }

        &::after {
          right: 3px;
          border-bottom-right-radius: 9px;
          border-top-right-radius: 9px;
          background-color: #848484;
        }
      }
    }

    .slider-wheel-top {
      color: #8a8a8a;
      display: flex;
      justify-content: space-between;
    }
  }

  &-controls {
    width: 250px;
    padding-bottom: 10px;
    display: flex;
    flex-direction: column;
    align-items: center;
  }

  &-slider-wheel {
    touch-action: none;
    position: relative;
    width: 210px;
    min-height: 20px;
    margin-top: 10px;
    margin-bottom: 20px;
    border-radius: 5px;
    border: 1px solid #2e2e2e;
    overflow: hidden;
    cursor: pointer;
    box-shadow: inset 25px 0px 25px -25px rgba(0, 0, 0, .8), inset -25px 0px 25px -25px rgba(0, 0, 0, .8);
    background-color: @panel-background-color;
    
    &:before {
      content: "";
      position: absolute;
      top: 0px;
      left: 0px;
      z-index: 2;
      width: 100%;
      height: 100%;
      border-radius: 5px;
      background: transparent linear-gradient(89deg, #050505 0%, #23232300 19%, #1B1B1B00 81%, #080808 100%) 0% 0% no-repeat padding-box;
    }
    
    .circles-list {
      position: absolute;
      left: -50%;
      display: flex;
      align-items: center;
      justify-content: space-between;
      height: 100%;

      .slider-circle {
        background: #464646;
        border-radius: 0px;
        border: solid 1px #383838;
        margin-right: 3px;
        height: 12px;
        width: 4px;
      }
    }
  }
}