.color-pickers-list-container {
  display: flex;
  width: 100%;

  .color-picker-container {
    height: 448px;
  }

  .colors-list {
    margin: 0;
    padding: 0;
    flex: 1;
    max-height: 450px;
    overflow: auto;

    .list-item {
      padding-top: .3rem;
      padding-bottom: .3rem;
      padding-left: 1rem;
      position: relative;
      cursor: pointer;

      .draggable-item {
        width: calc(100% - 20px);
        padding-top: 7px;
        padding-bottom: 7px;
        display: flex;
        justify-content: space-between;
        margin: 0;

        .expand-icon {
          margin: 0 1rem;
        }
      }

      .color-preview {
        width: 50px;
        height: 15px;
        border-radius: 5px;
        margin-right: 1rem;
      }

      &.selected {
        &::before {
          content: '';
          top: 0;
          left: 0;
          position: absolute;
          display: block;
          width: 4px;
          height: 100%;
          background-color: @presets-selected-background-color;
        }

        .color-preview {
          //border-color: @color-primary-color;
        }
      }
    }
  }
}