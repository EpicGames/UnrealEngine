.dial-wrapper-container {
  text-align: center;

  .proportional.icon {
    margin-left: 1rem;
  }
}

.dial-wrapper-block {
  width: 100%;
  height: 100%;
  display: flex;
  justify-content: space-evenly;
  align-items: center;
}

.dial-widget-wrapper {
  width: 295px;
  display: flex;
  flex-direction: column;
  align-items: center;
  
  .dial-label {
    margin: 0 0 10px;
    text-align: center;
  }

  & > svg {
    position: absolute;
    top: 2px;
    right: 16px;
  }

  .expand-icon {
    left: 16px;
  }
}

.dial-border-first {
  width: 226px;
}

.controls-block {
  .dial-border-first {
    width: 213px;
  }
}

.dial-border-first,
.dial-border-second {
  padding: 10px;
  border: 1px solid @widget-dial-border-color;
  border-radius: 50%;
}

.dial-border-first {
  background: @widget-dial-background-color;
}

.dial-border-second {
  background: @widget-dial-background-wrapper-color;
}

.dial-wrapper {
  touch-action: none;
  position: relative;
  display: flex;
  justify-content: center;
  align-items: center;
  width: calc(100% - @widget-title-height);
  height: calc(100% - @widget-title-height);
  border: 0px;
  flex-wrap: wrap;
  animation: colors 1s infinite;

  &:focus {
    outline: none;
  }

  &:before {
    content: "";
    position: absolute;
    z-index: 2;
    width: 100%;
    height: 100%;
  }
}

.dial-circle {  
  box-sizing: border-box;
  overflow: hidden;
  position: relative;
  display: flex;
  justify-content: center;
  align-items: center;
  width: 100%;
  height: 100%;
  border: 1px solid @widget-dial-border-color;
  border-radius: 50%;
  background-color: @panel-background-color;
  animation: colors 1s infinite;

  .svg-circle {
    stroke: url(#shape-gradient) #fff;

    &.range {
      stroke: #0067B4;
    }
  }

  .dial-ticks {
    .dial-tick:after {
      transition: initial;
    }
  }
}

.svg-circle {
  position: absolute;
  width: 100%;
  height: 100%;
}

.dial-line {
  position: absolute;
  bottom: 0px;
  width: 100%;
  height: 100%;
  transform-origin: center center;
  background-color: transparent;
  display: flex;
  align-items: center;
  justify-content: center;

  &::after {
    content: '';
    width: 3px;
    height: 8%;
    margin-top: 46%;
    border-radius: 0 0 20px 20px;
    background-color: @widget-dial-needle-color;
  }
}

.dial-center-wrapper {
  display: flex;
  justify-content: center;
  align-items: center;
  width: 55%;
  height: 55%;
  border: 1px solid @widget-dial-border-color;
  border-radius: 50%;
  background: @widget-dial-background-wrapper-color;

  .dial-center {
    display: flex;
    justify-content: center;
    align-items: center;
    flex-direction: column;
    width: 70%;
    height: 70%;
    border: none;
    border-radius: 50%;
    background-color: @widget-dial-background-color;
    box-shadow: 0px 0px 10px 2px @widget-dial-shadow-color;
    font-weight: bold;
    user-select: none; 

    .dial-label {
      display: none;
      color: @text-color;
      margin-bottom: 5px;
    }

    .input-field {
      width: 70px;
      border: 1px solid @widget-dial-border-color;
      color: @widget-dial-value-color;
    }
  }
}

.dial-value {
  z-index: 9;
}

.dial-tick {
  position: absolute;
  width: 40%;
  height: 40%;
  top: 30%;
  left: 30%;
  z-index: 5;
  overflow: visible;
  outline: solid 0px purple;
  display: flex;
  justify-content: center;

  &::after {
    content: "";
    position: absolute;
    top: 5px;
    width: 2px;
    height: 10px;
    transition: all 180ms ease-out;
    filter: blur(0.5px);

  }

  &.large::after {
    height: 40%;
    top: -65%;
    background: @widget-dial-tick-large;
  }

  &.short::after {
    height: 40%;
    top: -65%;
    background: @widget-dial-tick-short;
  }
}
