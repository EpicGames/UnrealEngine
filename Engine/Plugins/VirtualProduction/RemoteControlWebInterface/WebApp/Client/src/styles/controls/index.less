
@import './button.less';
@import './checkbox.less';
@import './dial.less';
@import './joystick.less';
@import './slider.less';
@import './toggle.less';
@import './vectorDrawer.less';
@import './colorPicker.less';
@import './colorPickerList.less';
@import './tabs.less';
@import './text.less';
@import './vector.less';
@import './timeControl.less';

.buttons-wrapper {
  display: flex;
  justify-content: center;
  margin-top: 10px;

  button {
    border: solid 1px @btn-widget-border-color;
    border-left: 0px;
    font-size: 14px;
    color: @btn-widget-color;
    background-color: @btn-widget-background-color;
    padding: 5px 10px;

    &:first-child {
      border-top-left-radius: 4px;
      border-bottom-left-radius: 4px;
      border-left: solid 1px #000;
    }

    &:last-child {
      border-top-right-radius: 4px;
      border-bottom-right-radius: 4px;
      
    }

    &:focus {
      outline: none;
    }
    
    &.selected {
      background-color: @btn-primary-selected-background-color;
      color: @btn-primary-selected-color;
      border-color: @btn-primary-selected-border-color;
    }

  }
}

.input-field-container {
  &::before {
    content: "";
    position: absolute;
    z-index: 1;
    width: 10px;
    height: 100%;
  }
}

.input-field {
  width: 85px;
  height: 25px;
  font-size: 15px;
  text-align: center;
  border-radius: @input-field-border-radius;
  background-color: @input-field-background-color;
  text-decoration: underline;
  text-decoration-style: dotted;
  text-underline-position: from-font;
  border-style: dotted;
  touch-action: none;
  color: #8a8a8a;
  padding: 0px;
  cursor: ew-resize;
}


.customstack-wrapper {

  .input-field {
    width: 120px;
    height: 25px;
    font-size: 15px;
    text-align: center;
    border-radius: 5px;
    background-color: @input-field-background-color;
    border: solid 1px @btn-widget-border-color;
    touch-action: none;
    cursor: ew-resize;
    text-decoration: none;
  }

  input {
    background-color: @input-field-background-color;
    border: solid 1px @btn-widget-border-color;
  }
}