.joystick-outer-wrapper {
  position: relative;
  width: 100%;
  display: flex;
  align-items: center;

  .reset-btn {
    position: absolute;
    right: 22px;
    top: 5px;
  }

  .controls-block {
    width: 100%;

    .inputs {
      display: flex;
      justify-content: center;
      
      .control {
        height: 30px;
        display: flex;
        align-items: center;
        justify-content: center;
        margin: 0 15px 0 0;

        .label {
          margin: 0 10px;
        }
      }
    }

    .slider {
      .slider-wrapper {
        margin-top: .5rem;
        width: 100%;
      }

      .centered {
        text-align: center;
      }

      .values {
        display: flex;
        justify-content: space-between;
        margin: .5rem 0;
      }
    }

    > .label {
      text-align: center;
      margin-bottom: 1rem;
      font-size: 15px;
    }
  }
}

.joystick-wrapper {
  width: 205px;
  height: 230px;
  display: flex;
  align-content: center;
  justify-content: space-evenly;
  flex-direction: column;

  .idle {
    transition: all 0.2s ease-in-out;
  }

  .joystick-value-wrapper {

    .joystick-value-label {
      outline: solid 0px red;
      width: 80px;
      margin: 5px auto 0px;
      text-align: center;
      display: none;
    }

    .joystick-value {
      width: fit-content;
      height: @widget-joystick-value-height;
      display: flex;
      flex-direction: row;
      justify-content: space-around;
      align-items: center;
      overflow: hidden;
      margin: 10px auto 0px;
      min-width: 80px;
      max-width: 185px;
      
      span {
        width: fit-content;
        margin: 0px 5px;
      }
    }

  }

 

  .joystick {
    touch-action: none;
    display: flex;
    justify-content: center;
    align-items: center;
    width: 75%;
    height: calc(75% - (@widget-joystick-value-height * 0.75));
    position: relative;
    background-color: @widget-joystick-background-color;
    border-radius: 50%;
    margin: 0 auto;
    border: @widget-joystick-outer-border;

    &::before {
      content: "";
      position: absolute;
      z-index: 5;
      width: 100%;
      height: 100%;
    }

    &::after {
      content: "";
      background: @widget-joystick-inner-circle-color;
      width: 100%;
      height: 100%;
      position: absolute;
      z-index: 1;
      border-radius: 50%;
      transform: scale(0.48);
      border: @widget-joystick-inner-circle-border;
      box-shadow: rgba(0, 0, 0, 0.5) 0px 3px 10px 0px;
    }

    .arrows {
      position: absolute;
      width: 100%;
      height: 100%;

      svg {
        position: absolute;
        font-size: 40px;
        color: @widget-joystick-arrow-color;
        min-width: 40px;

        &.fa-caret-up,
        &.fa-caret-down {
          left: 50%;
          transform: translateX(-50%);
        }

        &.fa-caret-left,
        &.fa-caret-right {
          top: 50%;
          transform: translateY(-50%);
        }

        &.fa-caret-right {
          right: 0;
        }

        &.fa-caret-down {
          bottom: 0;
        }
      }
    }
    .circle-wrapper {
      width: 100%;
      height: 100%;
      position: relative;
      z-index: 2;

      .circle {
        position: absolute;
        top: calc(50% - 32px);
        left: calc(50% - 32px);
        width: 60px;
        height: 60px;
        border-radius: 50%;
        background-color: @widget-joystick-foreground-color;
        box-shadow: rgba(0, 0, 0, 0.5) 0px 3px 6px 0px;
        z-index: 2;
        border: @widget-joystick-inner-border;
        display: flex;
        align-items: center;
        justify-content: center;

        &.animate {
          transition: transform 0.2s ease-out;
        }

        .label {
          font-size: 25px;
          color: @widget-joystick-label-color;
          text-align: center;
          white-space: nowrap;
          text-overflow: ellipsis;
          margin: 0px;
          font-weight: bolder;
        }
      }

    }

    .label {
      position: absolute;
      overflow: hidden;
    }

  }

}

.joystick-wrapper-block {
  width: 100%;
  display: flex;
  justify-content: center;
}

.step-wrapper {
  width: 100%;
  display: flex;
  justify-content: center;
  flex-direction: column;

  .label-wrapper {
    display: flex;
    justify-content: space-between;
    padding: 0 110px 0 30px;

    :first-child {
      padding: 0 20px 0 0;
    }
  }
}

.joystick-group {
  .slider-wrapper {
    width: 100%;
  }
}


.joystick-outer-wrapper,
.joystick-group {
  .slider-wrapper {
    width: 100%;
  }
}

.custom-sliders,
  .controls-block {
    .slider-row,
    .control {
      .title,
      .label {
        position: relative;
        display: flex;
        align-items: center;
        overflow: inherit;
      }

      .input-field-container {
        position: relative;

        &::after {
          content: "";
          position: absolute;
          left: 5px;
          top: 3px;
          height: 18px;
          width: 4px;
          border-radius: 20px;
          background-color: transparent;
        }
      }

      &:first-child {
        .input-field-container::after {
          background-color: #E93B3B;
        }
      }
      &:nth-child(2) {
        .input-field-container::after {
          background-color: #88BD49;
        }
      }
      &:nth-child(3) {
        .input-field-container::after {
          background-color: #0074D0;
        }
      }
      &:nth-child(4) {
        .input-field-container::after {
          background-color: #cdd000;
        }
      }
    }
  }