.slider-wrapper {
  width: calc(100% - 60px);
  outline: solid 0px red;
  display: flex;
  flex-direction: row-reverse;
  align-items: center;
  align-self: center;
  touch-action: none;

  .input-field {
    margin: 0 0 0 10px;
  }

  .slider-inner {
    height: fit-content;
    width: calc(100% - 20px);
    outline: solid 0px green;
    position: relative;
    margin: 0 auto;
  }

  .slider-clickable {
    width: 100%;
    height: @widget-slider-value-height + 10px;
    display: flex;
    flex-direction: column;
    justify-content: center;
    align-items: center;
  }
  
  .slider-block {
    width: 100%;
    height: @widget-slider-track-height;
    z-index: 2;
    position: relative;

    .slider {
      width: 100%;
      height: @widget-slider-track-height;
      cursor: pointer;
      background-color: @widget-slider-track-color;
      border-radius: 3px;

      &.touch-mode {
        cursor: initial;
      }
    }

    .circle-wrapper {
      pointer-events: none;
      width: 100%;
      position: relative;
      top: -10px;
      left: -(@widget-slider-thumb-size / 2);
      
      .circle {
        pointer-events: all;
        width: @widget-slider-thumb-size;
        height: @widget-slider-thumb-size;
        border-radius: 50%;
        cursor: pointer;
        background: @widget-slider-knob-color;
        touch-action: none;
      }
    }

    .range-fill {
      width: 100%;
      padding: 0;
      height: @widget-slider-track-height;
      background-color: @widget-slider-track-active-color;
      border-radius: 3px;
      position: relative;
      top: -@widget-slider-track-height;
      pointer-events: none;
    }
  }

  .grid-lines-wrapper {
    display: flex;
    justify-content: space-between;
    position: absolute;
    width: 100%;
    padding-top: 20px;
    z-index: 1;

    .grid-lines {
      width: 0;
      height: 7px;
      border: 1px solid @widget-slider-track-color;
      transition: all 0.2s ease;
    }

    .active {
      border: solid 1px @widget-slider-track-active-color;
    }

  }

  .label-wrapper {
    display: none;
    position: absolute;
    display: flex;
    justify-content: space-between;
    width: 100%;
    padding-top: 2px;
    pointer-events: none;
    margin-top: -5px;
  }  
}
