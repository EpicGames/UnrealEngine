.tabs-container {
  padding: 3px 10px;
  height: calc(100% - 12px);

  .tabs-header-elements {
    display: flex;
    justify-content: space-between;
    align-items: center;

    .tab-icon {
      font-size: 20px;
      padding: 7px;
      cursor: pointer;
    }

    .search-wrapper {
      margin-left: 10px;
      display: flex;
      align-items: center;

      .fa-search {
        position: relative;
        top: inherit;
        padding: 0;
      }

      .list-search {
        padding: 0 30px 0 10px;
      }
    }
  }

  .tabs-elements {
    width: 100%;
    position: relative;

    .selector {
      top: 4px;
      height: 25px;
      position: absolute;
      background-color: #1e1e1e;
      z-index: 1;
      transition: left 0.4s, width 0.4s;
      border-bottom: solid 2px #3C89F7;
    }
  }

  .tabs-header {
    position: relative;
    display: flex;
    justify-content: space-around;
    align-items: center;
    padding: 5px 0;
    width: 100%;
    height: 20px;

    &::after {
      content: '';
      position: absolute;
      width: calc(100% - 13px);
      height: 1px;
      background-color: #757575;
      bottom: -1px;
    }

    .tab {
      cursor: pointer;
      z-index: 2;
      padding: 7px;

      &:first-child {
        margin-left: 5px;
      }

      &:last-child {
        margin-right: 5px;
      }
      
      .icon {
        padding-right: 5px;
        font-size: 11px;
      }

      &.active {
        color: #FFF;
      }
    }
  }

  .tabs-content {
    margin-top: 0.5rem;
    height: calc(100% - 30px);

    .content {
      height: 100%;
      width: 100%;
    }
  }
}