.widget-text-wrapper {
  flex-grow: 1;

  .widget-text {
    width: 100%;
  }
}