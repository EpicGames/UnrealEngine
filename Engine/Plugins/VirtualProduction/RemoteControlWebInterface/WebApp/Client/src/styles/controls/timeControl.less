.time-control {
  display: flex;
  align-items: center;
  width: 100%;

  &.disabled {
    pointer-events: none;
    opacity: .5;
  }

  label {
    margin-right: 1rem;
  }

  .time-component {
    background-color: #121212;
    width: 100%;
    height: 35px;
    border-radius: 5px;
    display: flex;
    align-items: center;
    font-size: 25px;
    justify-content: space-between;

    .icon {
      font-size: 15px;
      cursor: pointer;

      &:first-child {
        margin-left: 10px;
      }

      &:last-child {
        margin-right: 10px;
      }
    }

    input::-webkit-outer-spin-button,
      input::-webkit-inner-spin-button {
        -webkit-appearance: none;
        margin: 0;
      }

    .time-input {
      padding: 0;
      width: 35px;
      text-align: center;
      font-size: 25px;
    }
  }
}