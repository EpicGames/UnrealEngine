@button-bevel-size: 7px;

.droppable-list.hidden {
  .switch {
    &::after,
    &::before {
      transition: none;
    }
  }
}

.toggle {
  position:relative;
  display:block;
  width: 200px;
  min-height: 140px;

  .title {
    display: -webkit-box;
    -webkit-line-clamp: 2;
    -webkit-box-orient: vertical;
    overflow: hidden;
    text-overflow: ellipsis;
    margin: 10px;
    text-align: center;
    cursor: inherit;
    height: 20px;
    height: 33px;
  }

  .switch {
    overflow: hidden;
    position: relative;
    width: calc(100% - 30px);
    height: 74px;
    border: 2px solid #2c2c2c;
    border-radius: 5px;
    background: #181818;
    cursor:pointer;
    margin: 0px 12px;

    &:before {
      content:'ON';
      display:block;
      position:absolute;
      height: calc(100% - 10px);
      text-align:center;
      line-height: 70px;
      width: calc(50% - 5px);
      top: 5px;
      left: 50%;
      background: #181818;
      color: #9d9d9d;
      border-radius: 5px;
      transition:.2s ease-in-out;
      display: flex;
      justify-content: center;
      align-items: center;
    }
    
    &:after {
      content:'OFF';
      display:block;
      position:absolute;
      height: calc(100% - 10px);
      text-align:center;
      line-height: 70px;
      width: calc(50% - 5px);
      bottom: 5px;
      left: 5px;
      background:#fff4d3;
      background: #202020;
      color: #9d9d9d;
      border-radius: 5px;
      transition:.2s ease-in-out;
      display: flex;
      justify-content: center;
      align-items: center;
    }
  }

  // TOGGLE ON STATE:
  &.checked  {
    .switch {
      &:before {
        background: #0074d7;
        color: @btn-primary-color;
      }
      &:after {
        background: #181818;
      }
    }
  }
}


.switch.vertical {
  input[type="checkbox"]{
    width: 30px;
    height: 140px;

    &:before {
      width: calc(100% - 6px);
      height: calc(50% - 5px);
      left: 1px;
      top: 1px
    }
    &:checked {
      &:before {
        width: calc(100% - 6px);
        height: calc(50% - 5px);
        left: 1px;
        top: 70px
      }
    }
    
  }
}