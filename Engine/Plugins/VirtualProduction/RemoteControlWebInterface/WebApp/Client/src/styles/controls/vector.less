.vector-control {
  display: flex;
  width: 100%;

  .vector-item {
    width: 100%;
  }

  &.disabled {
    .slider-control {
      opacity: 0.5;
      pointer-events: none;
    }
  }

  .reset-icon {
    cursor: pointer;
    padding: 5px;
  }

  .title {
    margin-right: 1rem;
    margin-top: 5px;
    min-width: 55px;
  }

  .keys-row {
    display: flex;
    justify-content: space-between;

    .key {
      display: flex;
      align-items: center;
      min-width: 130px;
      position: relative;
      padding-left: 0.5rem;

      &.active {
        outline: 1px solid #5A91F7;
      }

      .property {
        min-width: 35px;
      }

      &::before {
        content: '';
        position: absolute;
        display: block;
        height: 100%;
        width: 2px;
        left: 0;
      }

      &:nth-child(1) {
        &::before {
          background-color: #ff0000;
        }
      }

      &:nth-child(2) {
        &::before {
          background-color: #00ff00;
        }
      }
      
      &:nth-child(3) {
        &::before {
          background-color: #0000ff;
        }
      }

      input::-webkit-outer-spin-button,
      input::-webkit-inner-spin-button {
        -webkit-appearance: none;
        margin: 0;
      }

      .input-field {
        border: none;
        text-align: right;
      }
    }
  }

  .slider-control {
    position: relative;
    margin-top: 1rem;
    background-color: #464646;
    height: 10px;
    border: 1px solid rgba(255, 255, 255, 0.2);
    touch-action: none;
    overflow: hidden;
    cursor: pointer;

    .dash {
      display: inline-block;
      height: 100%;
      width: 8px;
      border-left: 1px solid rgba(255, 255, 255, 0.2);
    }
  }

  .vector-container {
    position: relative;
  }

  .pointer {
    position: absolute;
    width: 10px;
    height: 20px;
    background-color: #464646;
    border: 1px solid rgba(255, 255, 255, 0.2);
    top: -6px;
    left: calc(50% - 6px);
    touch-action: none;
    pointer-events: none;
  }
}