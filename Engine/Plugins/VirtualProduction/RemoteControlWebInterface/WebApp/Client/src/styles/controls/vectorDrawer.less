.customstack-wrapper.drawer-shown {
  height: calc(100% - @modal-controls-height);
  overflow: auto;
}

.tabs-open {
  .vector-drawer-wrapper {
    width: calc(100% - 400px);

    @media only screen and (max-device-width: 1366px) {
      width: calc(100% - 225px);
    }
  }
}

.vector-drawer-wrapper {
  width: 100%;
  height: @modal-controls-height;
  display: flex;
  justify-content: center;
  align-items: center;
  flex-direction: column;
  position: fixed;
  bottom: 0;
  right: 0;
  background-color: @modal-background-color;
  overflow: hidden;
  transition: all 0.3s ease;
  z-index: 99;
  padding: 20px;
  box-sizing: border-box;
  border-top: 1px solid #353944;
  border-left: 1px solid #353944;

  .slider-clickable  {
    .slider-block {
      height: 8px;
      margin: 0;

      .slider {
        height: inherit;
      }
      
      .range-fill {
        height: inherit;
      }
    }
  }

  .joystick-outer-wrapper .joystick .label {
    display: initial;
    margin: 0;
  }

  .controls-block {
    height: 100%;
    width: 100%;

    .sliders-component-wrapper {
      .slider-row {
        svg {
          display: none;
        }
      }
    }
  }

  .drawer-header {
    position: relative;
    display: flex;
    align-items: center;
    width: 100%;
    max-width: 1400px;
    margin: 0 0 10px;
    box-sizing: border-box;

    .header-block {
      display: flex;
      align-items: center;
      justify-content: center;
      flex: 1;
      text-align: center;

      & > span {
        font-size: 24px;
      }

      .reset-btn {
        margin: 0 0 0 10px;
      }
    }

    .btns {
      display: flex;
      align-items: center;
      justify-content: flex-end;
    }
    
    .close-btn {
      margin-left: 10px;
      font-size: 18px;
    }
  }

  .modes {
    position: relative;
    display: flex;
    align-items: center;
    padding: 0 20px 0 0;

    .mode {
      background-color: #555;
      padding: 8px;
      cursor: pointer;
      border: solid 1px #000;

      &:first-child {
        border-top-left-radius: 4px;
        border-bottom-left-radius: 4px;
      }

      &:last-child {
        border-top-right-radius: 4px;
        border-bottom-right-radius: 4px;
      }

      &.selected {
        background: #50b6f9;
      }
    }

    @media only screen and (min-device-width: 1366px) {
      padding: 0;
    }
  }

  .lock-btn {
    width: 30px;
    height: 30px;
    position: absolute;
    top: 0px;
    right: 40px;
  }

  .dial-widget-wrapper {
    & > .dial-label {
      display: none;
    }

    .dial-center-wrapper {
      .dial-label {
        display: block;
      }
    }

  }
}

.sliders-component-wrapper {
  width: 100%;
  height: 100%;
  display: flex;
  justify-content: center;
  align-items: center;
  flex-direction: column;

  .title {
    min-width: 30px;
    width: 30px;
  }
}