.list {
  border-right: solid 1px @lists-border-color;
  height: 100%;
  overflow-y: auto;

  .list-title {
    background-color: @lists-item-highlight-background-color;
    padding: 10px;
    display: flex;
    justify-content: space-between;
    position: sticky;
    top: -1px;
    cursor: pointer;
    z-index: 10;
  }

  & > ul > li > ul {
      overflow: hidden;
      max-height: 1000px;

      transition: all 0.2s ease;
      cursor: pointer;

      & > li {
        padding: 10px 20px;
        transition:  all 0.2s ease;

        @media (hover: hover){
          &:hover {
            color: @lists-item-selected-color;
            background-color: lighten(@lists-item-selected-background-color, 10);
          }
        }

        &.selected {
          color: @lists-item-selected-color;
          background-color: @lists-item-selected-background-color;
        }
      }
  }

  .items-list {
    display: none;

    &.expanded {
      display: block;
    }

    .dragging-overlay {
      display: flex;
      justify-content: center;
      width: 100%;
      padding: 5px;
      border: 1px solid @color-primary-color;
      background-color: @panel-background-color;

      .title,
      .list {
        background-color: inherit;
      }
    }

    .btn-wrapper {      
      position: relative;
      width: 200px;

      .widget-button-wrapper {
        height: 49px;

        .widget-button {
          height: 42px;
          top: 2px;
          left: 50%;
          transform: translateX(-50%);
          margin: 0;
          line-height: 18px;
  
          display: -webkit-box;
          -webkit-line-clamp: 2;
          -webkit-box-orient: vertical;  
          overflow: hidden;
          text-overflow: ellipsis;
        }
      }
    }
  }
  
  .selected {
    background-color: @lists-item-selected-background-color;
  }

  ul {
    list-style: none;
    padding-inline-start: 0px;
    margin: 0px;

    li {
      padding: 0px;
      
    }
  }
}