.modal {
  box-sizing: border-box;
  margin: 0;
  padding: 0;
  color: @color-primary-text;
  font-size: 14px;
  width: auto;
  margin: 0px;
  padding-bottom: 24px;
  margin-bottom: 50px;
  width: 520px;
  background-color: @modal-background-color;
  outline: none;
}

.rename-modal {
  background-color: @modal-background-color;
  padding: 20px;
  display: flex;
  flex-direction: column;
  align-items: center;
  position: relative;
  margin-top: -200px;

  .btn {
    width: 100px;
    height: 40px;
  }

  input {
    font-size: 30px;
    text-align: center;
    background: none;
    outline: none;
    border-bottom: 2px solid white;
    border-top: none;
    border-left: none;
    border-right: none;
    border-radius: 0px;
  }
}

.fullscreen.precision {
  z-index: 990;
}

.precision-modal {
  padding-bottom: 1rem;
  width: 800px;
  min-height: 500px;
  background-color: @modal-background-color;
  display: flex;
  flex-direction: column;
  align-items: center;
  position: relative;

  .main {
    display: flex;
    flex-direction: column;
    height: 100%;
    justify-content: center;

    &.color-picker {
      width: 100%;
      justify-content: flex-start;
      margin-top: 1rem;

      .btn-group {
        display: flex;
        margin-left: auto;
        margin-right: 1rem;

        .btn {
          padding: 8px 10px;
        }
      }

      .color-container {
        text-align: center;
        display: flex;
        flex-direction: column;
        height: 100%;
        justify-content: space-between;
      }

      .color-picker-container {
        height: initial;
      }
    }
  }

  .color-picker-slider-wheel {
   min-height: 40px;

   .slider-circle {
     width: 14px;
     height: 14px;
   }
  }

  .slider-wheel-rows {
    flex: 1;
    margin-left: 1rem;

    .slider-wheel-row {
      display: flex;
      align-items: center;
      margin-bottom: 1.5rem;

      .precision-icon {
        font-size: 30px;
      }

      .color-picker-slider-wheel {
        margin: 0;
        width: 70%;
        height: 20px;
      }

      .input-field {
        width: 80px;
        margin: 0 1rem;
      }
    }

    .slider-label {
      display: flex;
      width: 50px;
      height: 25px;
      align-items: center;
      justify-content: center;
      background-color: #171717;
    }
  }

  .close-modal {
    z-index: 1;
    top: 11px;
  }

  .label {
    font-size: 18px;
    width: 100%;
    top: 0;
    height: 50px;
    display: flex;
    align-items: center;
    justify-content: center;
    background: #171717;

    p {
      white-space: nowrap;
      overflow: hidden;
      text-overflow: ellipsis;
      width: 770px;
      margin: 0;
      padding-right: 40px;
      padding-left: 15px;
      text-align: center;
    }
  }

  .precision-icon {
    font-size: 40px;
    cursor: pointer;
    margin: 0 1rem;
  }

  .step-label {
    text-transform: uppercase;
    text-align: center;
  }

  .sensitivity-block {
    background: #1f1f1f;
    padding: 1rem;
    margin-top: 3rem;
  }

  .row {
    display: flex;
    margin: 1rem 0;
    width: 100%;
    display: flex;
    align-items: center;
    justify-content: center;
    min-height: 40px;

    &.top {
      .slider-wrapper  {
        margin: 0 2rem;
      }

      .input-field {
        width: 77px;
      }
    }

    &.m-0 {
      margin: 0;
    }

    .color-picker-slider-wheel {
      width: 100%;
    }

    .limits {
      font-size: 14px;
      margin: 0 .3rem;
      display: flex;
      flex-direction: column;
      align-items: center;
    }

    .slider-wrapper {
      width: 100%;

      .slider-inner {
        margin: 0;
        width: 100%;
      }
    }

    &.space {
      justify-content: space-between;

      .limits .input-field {
        width: 77px;
        margin-top: 5px;
      }

      .step-input {
        display: flex;
        align-items: center;
        margin-top: 21px;

        .icon {
          font-size: 25px;
          margin: 0 1rem;
        }
      }
    }

    &.start {
      align-items: flex-start;
    }
  }

  .sensitivity-slider {
    width: calc(100% - 35px);
    margin: 3rem auto 0;
  }

  .toggle-checkbox {
    display: flex;
    align-items: center;
    margin-left: 1rem;

    .toggle-label {
      margin-left: .5rem;
    }
  }
}

.alert-modal {
  background-color: @modal-background-color;
  padding: 20px;
  display: flex;
  flex-direction: column;
  align-items: center;
  position: relative;
  margin-top: -200px;
  min-width: 30%;

  .alert-text {
    width: fit-content;
    font-size: 1.5vw;
    margin: 20px;
  }

  .btn {
    width: 100px;
    height: 40px;
    margin: 5px;
  }
}

.icon-modal {
  height: 70%;
  width: 70%;
  background-color: @modal-background-color;
  padding: 20px;
  display: flex;
  flex-direction: column;
  align-items: center;
  justify-content: center;
  position: relative;
  
  .search-position {
    top: 5%;
  }

  .icon-search {
    font-style: large;
  }

  .icon-wrapper {
    width: 90%;
    height: 65%;
    overflow-y: scroll;
    display: flex;
    flex-wrap: wrap;
    justify-content: flex-start;
    align-items: center;
    margin-top: 10%;

    .icon-block {
      display: flex;
      flex-direction: column;
      justify-content: space-evenly;
      align-items: center;
      width: 70px;
      height: 90px;
      box-sizing: border-box;
      margin: 10px;

      .icon {
        width: 40px;
        height: 40px;
        cursor: pointer;
      }
  
      .icon-text {
        padding: 0 3px;
        font-size: 10px;
        height: 28px;
        text-align: center;
        overflow: hidden;
      }
    }

    .selected {
      background-color: @btn-primary-background-color;
      border-radius: 4px;
      color: #FFF;
    }
  }

  .name-text {
    margin-top: 20px;
  }

  .btn {
    width: 200px;
    margin: 20px 20px 0px 20px;
  }
}

input.search-input {
  font-size: 30px;
  text-align: center;
  background: none;

  outline: none;
  border-bottom: 2px solid white;
  border-top: none;
  border-left: none;
  border-right: none;
  border-radius: 0px;
  padding-left: 35px;
}

.close-modal {
  position: absolute;
  right: 20px;
  top: 20px;
  font-size: 20px;
  cursor: pointer;
}

.search-modal {
  position: relative;
  background-color: #242424;
  width: 70%;
  height: 70%;
  padding: 20px;

  .search-wrapper {
    text-align: center;

    .reset-search-icon {
      right: initial;
    }

    .fa-search {
      font-size: 35px;
    }
  }

  .close-modal {
    z-index: 1;
  }

  .search-field {
    position: relative;

    .icon-search {
      position: absolute;
      left: 8px;
      top: 6px;
      position: absolute;
    }

    input {
      width: calc(100% - 40px);
      padding: 5px;
      padding-left: 30px;
    }
  }

  .search-content {
    margin-top: 1rem;
    overflow: auto;
    height: calc(100% - 60px);

    .search-item {
      display: flex;
      align-items: center;
      font-size: 16px;      
      cursor: pointer;
      border-bottom: 1px solid rgba(255, 255, 255, .2);
      margin-right: 1rem;
      padding: .5rem 0 .5rem .5rem;
      
      &:hover {
        background: rgba(90, 90, 90, 0.4);
      }

      .thumbnail {
        max-width: 80px;
      }

      .path {
        width: calc(55% - 20px);
        overflow: hidden;
        text-overflow: ellipsis;
      }

      .title {
        margin: 0 1rem;
        width: 40%;
        overflow: hidden;
        text-overflow: ellipsis;
      }
    }

    .content-placeholder {
      font-size: 30px;
      height: 100%;
      display: flex;
      align-items: center;
      justify-content: center;
      opacity: 0.5;

      .loader-icon {
        margin-left: 1rem;
      }
    }
  }
}