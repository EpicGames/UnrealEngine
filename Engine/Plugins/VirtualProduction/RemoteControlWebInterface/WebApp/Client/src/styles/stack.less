@modal-controls-height: 320px;

.customstack-wrapper {
  box-sizing: border-box;
  flex-direction: column;
  padding: 10px;
  padding-right: 0;
  height: 100%;
  max-width: 980px;
  width: 100%;

  .btn-revert {
    margin: 1rem auto 0;
    width: 50%;
    background-color: #E93B3B;
    color: #fff;

    &:disabled {
      background-color: #5a5a5a;
      cursor: not-allowed;
      color: #8a8a8a;
    }
  }

  &.customstack-list {
    max-width: 1085px;
  }

  &.drag {
    .draggable-item {
      transition: top .2s cubic-bezier(.2, 0, 0, 1), left .2s cubic-bezier(.2, 0, 0, 1);
    }

    .joystick {
      pointer-events: none;
    }
  }

  .droppable-root {
    min-height: calc(100vh - 90px);
    overflow: hidden;
  }

  svg {
    cursor: pointer;
  }

  .navigation-list {
    width: calc(100% - 5px);
    height: fit-content;
    display: flex;

    &-panels {
      position: relative;
      width: 100%;

      .hidden {
        width: 100%;
        visibility: hidden;
        position: absolute;
        pointer-events: none;
        top: 0;
        max-height: calc(100vh - 85px);
      }
    }

    .action-button {
      background-color: #222;
      color: #d3d3d3;
      padding: 5px;
      margin-left: 0;
    }
  }

  .draggable-item.label {
    display: flex;
    justify-content: center;
    margin: 30px 0px 20px 0px;
    font-weight: bold;
    font-size: 18px;
  }

  .draggable-item.vector-dials {
    margin: 30px 5px 20px 5px;
    display: flex;
    justify-content: center;
    min-height: 315px;
  }

  button {
    cursor: pointer;
    background: transparent;
    border: 0px;
    color: @color-primary-text;
    margin: 0px 7px;

    &:hover {
      color: @btn-primary-selected-color;
    }
  }

  button:focus {
    outline: none;
  }

  .navigation-btn {
    height: 35px;
    margin: 0 10px 0 0;
    background-color: @background-button-color;
    color: #fff;
  }

  .navigation {
    min-width: @navigation-width;
    height: 100%;
    box-sizing: border-box;
    display: flex;
    flex-direction: column;
    padding: 0 10px 0 0;
    overflow-y: auto;
    overflow-x: hidden;

    .navigation-row {
      width: 100%;
      height: 40px;
      display: flex;
      justify-content: flex-start;
      align-items: center;
      margin: 10px 0;

      button {
        width: @navigation-btn-width;
      }

      p {
        margin: 0 5px;
        max-height: 35px;
        overflow: hidden;
        text-align: start;
      }

      .navigation-icon {
        width: 15px;
        height: 15px;
      }

      input:checked + .slider {
        background-color: @presets-selected-background-color;
        width: 60px;
      }

    }

  }

  .custom-sliders {
    position: relative;
    display: flex;
    flex-direction: column;
    justify-content: center;
    flex-direction: column;
    width: 100%;

    .expand-icon {
      padding: 0;
      margin: 0;
      margin-right: 1rem;
    }

    .proportional.icon {
      position: absolute;
      right: 41px;
      top: 50%;
      transform: translateY(-50%);
    }

    .reset-sliders {
      position: absolute;
      right: 22px;
      top: 50%;
      transform: translateY(-50%);
    }
  }

  .joystick-row {
    margin: 10px 0 0 0;
    justify-content: flex-start;

    .value-info {
      position: relative;
      display: flex;
      justify-content: flex-start;
      align-items: center;
      flex-direction: row;
      margin-right: 15px;

      .axis-title {
        position: absolute;
        left: 5px;
        width: 40px;
        text-align: start;
        padding-left: 8px;

        &::before {
          content: "";
          position: absolute;
          left: 0px;
          top: 0px;
          height: 18px;
          width: 4px;
          border-radius: 20px;
          background-color: transparent;
        }

        &.X-axis::before,
        &.Roll-axis::before {
          background-color: #E93B3B;
        }
        &.Y-axis::before,
        &.Pitch-axis::before {
          background-color: #88BD49;
        }
        &.Z-axis::before,
        &.Yaw-axis::before {
          background-color: #0074D0;
        }
      }

      input {
        width: 155px;
        padding: 0px 10px 0px 50px;
        cursor: ew-resize;
      }
    }

    .control-buttons {
      display: flex;
      align-items: center;
      justify-content: center;
      margin: 0 0 0 auto;
    }

    .lock-image {
      width: 25px;
      height: 25px;
      margin: 0 10px 0 0;
    }

    .gamepad {
      position: relative;
      width: 30px;
      height: 30px;
      margin: 0 10px 0 0;

      &.selected {
        color: #0074D0;
      }
    }

  }

  .image-dropdown-row {
    width: 100%;
    display: flex;
    flex-direction: row;
    overflow-x: auto;
    overflow-y: hidden;
    overflow: auto;
  
    .dropdown-image {
      box-sizing: border-box;
      margin: 0 30px 0 0;
      border: solid 2px rgba(0, 0, 0, 0);
      border-radius: 10px;
    }
  
    .selected-dropdown-image {
      border: solid 2px @secondary-selected-color;
    }
  }
}

.droppable {  
  &.droppable-panel {
    position: relative;
    background-color: @panel-background-color;
    min-height: 200px;
    padding: 10px 0px;
    margin-right: 10px;
    border-radius: 10px;
    margin-bottom: 15px;
    box-shadow: 0px 0px 10px 2px rgba(0,0,0,0.4);

    > .title {
      text-align: center;
      font-size: 16px;
      text-transform: uppercase;
      color: #FFF;
      padding: 10px;
      background-color: @panel-title-background-color;
      border-top-left-radius: 10px;
      border-top-right-radius: 10px;
    }
  }

  &.has-title {
    padding: 0px 0px 10px 0px;
  }

  .template-property-select-wrapper {
    position: sticky;
    top: -1px;
    z-index: 999;

    display: flex;
    justify-content: center;
    align-items: center;
  }

  .template-property-select-inner-wrapper {
    display: flex;
    justify-content: center;
    align-items: center;

    &::after {
      content: '';
      
      width: 0;
      height: 0;
      border-left: 8px solid transparent;
      border-right: 8px solid transparent;
      border-top: 8px solid #ccc;
      clear: both;
      z-index: 999;
      margin-left: -23px;
    }
  }

  .template-property-select {
    background-color: #222;
    border: solid 1px #000;
    color: #b5b5b5;
    padding: 8px 30px 8px 10px;
    border-radius: 5px;
    min-width: 100px;
    -webkit-appearance:none;
    width: 500px;
    outline: none;

    
  }

  &.dragging {
    border: 1px solid @color-primary-color;
  }

  &.droppable-list {
    width: calc(100% - 5px);
    min-height: 200px;
    padding: .5rem 0;

    &.hidden {
      visibility: hidden;
      width: 100%;
      height: 100%;
      position: absolute;
      pointer-events: none;
      top: 0;
    }
  }

  li {
    list-style: none;
  }

  .draggable-item {
    position: relative;
    margin: 10px 1px 0;
    top: 0;
    left: 0;

    &.selected {
      outline: 1px solid @color-primary-color;
      background-color: fade(@color-primary-color, 30%);;
    }
  
    .navigation-row {
      width: 100%;
      height: 40px;
      display: flex;
      justify-content: flex-start;
      align-items: center;
      margin: 10px 0;
  
      button {
        width: @navigation-btn-width;
      }
  
      p {
        margin: 0 5px;
        max-height: 35px;
        overflow: hidden;
        text-align: start;
      }
  
      .navigation-icon {
        width: 15px;
        height: 15px;
      }
  
      input:checked + .slider {
        background-color: @presets-selected-background-color;
        width: 60px;
      }
  
      button {
        border: solid 1px @panel-background-color;
        border-radius: 5px;
        cursor: pointer;
      }
  
      .navigation-btn {
        height: 35px;
        margin: 0 10px 0 0;
        background-color: @background-button-color;
        color: #fff;
  
        &.selected {
          background-color: @presets-selected-background-color;
        }
      }
    }
  
    .drag-handle {
      display: none;
      position: absolute;
      top: 50%;
      left: 0px;
      transform: translateY(-50%);

      
      svg {
        font-size: 14px;
      }
  
      &.right {
        left: 100%;
      }
    }
  
    &.edit {
      padding-left: 15px;
  
      > .drag-handle {
        display: block;
      }
    }
  
    &.dragging {
      visibility: visible;

      &:after {
        content: '';
  
        position: absolute;
        display: block;
        border: 1px solid @color-primary-color;
        border-radius: 10px;
        background-color: @panel-background-color;
        z-index: -1;
        width: calc(100% + 5px);
        height: calc(100% + 5px);
        left: -5px;
        top: -3px;
      }
    }
  }
}

.unbound-property {
  display: flex;
  align-items: center;
  justify-content: center;

  svg {
    position: relative !important;
    margin: 0 5px 0 0;
    color: #e2ee2d;
    cursor: default;
  }
}

.tabs-open {
  .customstack-wrapper {
    .value-info {

      .Roll-axis,
      .Pitch-axis,
      .Yaw-axis {
        & + .input-field-container input {
          padding-left: 45px;
        }
      }

      input {
        width: 120px;
        padding-left: 25px;
        padding-right: 2px;
      }
    }
  }
}

.asset-item {
  padding: 15px;
  display: flex;
  align-items: center;
  margin-right: 10px;

  .input-title {
    width: 100%;
    display: flex;
    align-items: center;

    input {
      width: calc(100% - 10px);
    }
  }

  .image {
    margin-right: 1rem;
  }

  .label {
    margin-right: 1rem;
    width: 125px;
    min-width: 125px;
    max-height: 34px;
    overflow: hidden;
    text-align: start;
    font-size: 14px;
    line-height: 16px;
  }

  .asset-icon {
    cursor: pointer;
    padding: 5px;
    margin-left: auto;
    margin-right: 1rem;
    font-size: 18px;
  }
}