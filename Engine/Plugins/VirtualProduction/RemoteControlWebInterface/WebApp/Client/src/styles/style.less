@font-face {
  font-family: 'Font Awesome 5 Free';
  font-style: normal;
  font-weight: 900;
  font-display: block;
  src: url("../webfonts/fa-solid-900.eot");
  src: url("../webfonts/fa-solid-900.eot?#iefix") format("embedded-opentype"), url("../webfonts/fa-solid-900.woff2") format("woff2"), url("../webfonts/fa-solid-900.woff") format("woff"), url("../webfonts/fa-solid-900.ttf") format("truetype"), url("../webfonts/fa-solid-900.svg#fontawesome") format("svg"); }

  
@import './animations.less';
@import './theme.less';
@import './topBar.less';
@import './stack.less';
@import './controls/index.less';
@import './actors/index.less';
@import './modal.less';
@import './lists.less';
@import './swtich.less';
@import './tablet-specific.less';

* {
  -webkit-user-select: none;
}

::-webkit-scrollbar {
  width: 10px;
  background-color: @scrollbar-thumb-color; 
  border-radius: 10px;
}

::-webkit-scrollbar-track {
  border-radius: 10px;
  background-color: @scrollbar-thumb-color;
}

::-webkit-scrollbar-thumb {
  background: @scrollbar-track-color;
  border-radius: 10px;
}

html, body, #app, #root {
  margin: 0px;
  padding: 0px;
  height: 100%;
  width: 100%;
  background: @tabs-content-background-color;
  color: @color-primary-text;
  font-family: @font-primary-family;
  font-size: 14px;
  touch-action: none;
}

#app {
  width: 100%;
  height: 100%;
  transition: all 0.5s ease;
  overflow: hidden;
  position: relative;
  outline: none;

  &.locked {
    #top-bar {
      pointer-events: none;

      .locked-mode {
        pointer-events: all;
      }
    }

    &.tabs-open {
      .droppable-root  {
        pointer-events: none;
      }
    }

    &.tabs-closed {
      .customstack-wrapper:not(.customstack-list) {
        .droppable-root > div {
          pointer-events: none;
        }
      }

      .customstack-list {
        .navigation-list {
          .navigation,
          .droppable-root > div {
            pointer-events: none;
          }
        }
      }
    }
  }

  .signal-icon {
    position: absolute;
    bottom: 20px;
    right: 35px;
    height: 20px;

    display: flex;
    align-items: flex-end;
    justify-content: flex-end;

    &.normal {
      span {
        background-color: #ccb400;
      }
      
      .bar-4 {
        opacity: 0.2;
      }
    }

    &.bad {
      span {
        background-color: #cc1b00;
      }

      .bar-3,
      .bar-4 {
        opacity: 0.2;
      }
    }

    span {
      display: inline-block;
      width: 6px;
      margin-left: 2px;
      transform-origin: 100% 100%;
      border-radius: 2px;
    }

    .bar-1 {
      height: 25%;
    }
    .bar-2 {
      height: 50%;
    }
    .bar-3 {
      height: 75%;
    }
    .bar-4 {
      height: 100%;
    }
  }
  
  .empty-tab-container {
    display: flex;
    align-items: center;
    justify-content: center;
    background-color: #202020;
    margin: 20px;
    height: calc(100% - 80px);
    max-width: 980px;
    width: 100%;
    overflow: auto;

    .screens-list {
      padding: 0;
      margin: 0;
      list-style: none;
      height: 100%;
      display: flex;
      flex-direction: column;
      justify-content: center;

      .screen-type-item {
        margin-bottom: 1.5rem;
        min-width: 300px;

        &:last-child {
          margin-bottom: 0;
        }

        .btn {
          width: 100%;
          margin: 0;
          font-size: 20px;
        }
      }
    }
  }
}

.locked-layer {
  width: 100%;
  height: calc(100% - 50px);
  position: fixed;
  top: 50px;
  left: 0px;
  background-color: rgba(0,0,0,0.5);
  z-index: 9999;
  display: flex;
  justify-content: center;
  align-items: center;
  //opacity: 0.5;
  font-size: 72px;
  color: @widget-slider-knob-color;
}

#tabs-drawer-layout {
  position: absolute;
  top: 50px;
  overflow: hidden;
  width: 400px;
  height: calc(100% - 50px);
  right: auto;
  left: -400px;
  transition: left 0.2s ease;
  display: flex;
  flex-direction: column;

  @media only screen and (max-device-width: 1366px) {
    left: -225px;
    width: 225px;
  }

  & > div,
  & > ul {
    width: 100%;
  }

  .tab-content {
    height: 50%;
    overflow: hidden;
  }

  .drawer-nav {
    display: flex;
    height: 34px;
    align-items: center;
    padding-left: 16px;
    font-size: 15px;
    box-sizing: border-box;

    &-tab {
      padding-right: 20px;
      position: relative;
      left: 0;
      top: 0;
      right: 0;
      cursor: pointer;
      transition: width .3s,right .3s;
      white-space: nowrap;

      &:last-child {
        padding-right: 0;

        &::after {
          content: '';
          position: absolute;
          height: 4px;
          background-color: @secondary-selected-color;
          width: inherit;
          bottom: -6px;
          right: inherit;
        }
      }

      &.active {
        &:nth-child(1) ~ li:last-child {
          right: 190px;
          width: 52px;
        }
      
        &:nth-child(2) ~ li:last-child {
          right: 120px;
          width: 26px;
        }
      
        &:nth-child(3) ~ li:last-child {
          right: 72px;
          width: 52px;
        }

        &:nth-child(1) {
          width: 52px;
        }
      
        &:nth-child(4) {
          right: 1px;
          width: 70px;
        }
      }
    }
  }

  .nav-list-container {
    height: 100%;
  }
  .nav-list {
    height: calc(100% - 35px);
    overflow-x: hidden;
  }

  .nav-properties {
    border-top: 1px solid #353944;
    overflow: auto;
    height: calc(50% - 35px);

    .properties-title {
      background: #303030;
      padding: 10px;  
      text-align: center;
      font-weight: bold;
      border-top: solid 2px #191919;
    }
    .properties-list {
      margin: 15px 0;
      padding: 0px 5px 0px 15px;

      .list-property {
        .properties-field {
          margin-bottom: 5px;
          margin-left: 10px;
        }
      }
    }
  }

  .properties-field {
    display: grid;
    grid-template-columns: 135px auto;
    margin: 0 0 15px;
    padding: 0 10px 0 0;
    align-items: center;

    &:last-child {      
      margin: 0;
    }

    .checkbox {
      width: auto;
    }

    .dropdown {
      align-self: flex-start;
    }

    & > label {
      padding: 0 25px 0 0;
    }

    &.line {
      display: flex;
      align-items: center;

      label {
        margin: 0 0 5px;
      }
  
      .icon {
        cursor: pointer;
        margin-left: 10px;
      }
    }

    .vector-btn-group {
      .btn {
        padding: 8px;
      }
    }
  }

  .properties-grid {
    .btn-container {
      margin-top: 1rem;
      display: flex;
      justify-content: center;
    }

    input {
      width: 100%;
    }

    .property {
      position: relative;
      margin: 10px 0;
      padding: 0 15px 0 15px;

      .checkbox {
        position: absolute;
        left: 150px;
        opacity: 0;
        width: 20px;
        height: 20px;
        margin: 0;
        z-index: 1;
        cursor: pointer;
      
        & + label {
          position: relative;
          padding: 0;
        }
      
        & + label:before {
          content: '';
          position: absolute;
          left: 100%;
          display: inline-block;
          vertical-align: text-top;
          width: 20px;
          height: 20px;
          background-color: @input-field-background-color;
          border: solid 1px #141414;
          box-sizing: border-box;
        }
      
        &:checked + label:after {
          content: '';
          position: absolute;
          left: calc(100% + 5px);
          top: 9px;
          background: white;
          width: 2px;
          height: 2px;
          box-shadow: 
            2px 0 0 white,
            4px 0 0 white,
            4px -2px 0 white,
            4px -4px 0 white,
            4px -6px 0 white,
            4px -8px 0 white;
          transform: rotate(45deg);
        }
      }

    }
  }
    
  .types-selection {
    margin: 15px 0;
    padding: 0 5px 0 15px;
  }

  .items-list, .layout-widgets {
    .btn-wrapper {
      margin: 0 auto;
      width: 200px;
      height: 48px;
      margin: 5x;
      border: 1px solid transparent;
  
      .widget-button-wrapper {
        width: 100%;
        overflow: initial;
  
        .widget-button {
          height: 40px;
          top: 2px;
          left: 50%;
          transform: translateX(-50%);
          margin: 0;
          line-height: 18px;
  
          overflow: hidden;
          display: -webkit-box;
          -webkit-line-clamp: 1;
          -webkit-box-orient: vertical; 
  
          &:active {
            top: 4px !important;
          }
        }
      }
    }

    .draggable-widget {
      background-color: #484546;
      border: solid 1px #565656;
      border-radius: 4px;
      display: flex;
      justify-content: space-between;
      min-height: 38px;
      padding: 0 5px 0 35px;
      align-items: center;
      margin: 10px;
      position: relative;

      &.selected {
        outline: 1px solid #5abaf9;
        background-color: rgba(90, 186, 249, 0.3);
      }

      > img {
        position: absolute;
        left: 0;
      }

      &.control-widget::before {
        pointer-events: none;
        opacity: 0;
        content: attr(data-tooltip);
        position: absolute;
        top: -20px;
        z-index: 10;
        margin: auto;
        padding: 2px;
        border-radius: 2px;
        background-color: rgba(240, 230, 140, 1);
        font-size: 12px;
        color: #222;
        transform-origin: bottom;
        transition: opacity 0.2s ease;
        transition-delay: 0s;
      }

      &.control-widget:hover::before {
        opacity: 1;
        transition: opacity 0.2s ease;
        transition-delay: 1s;
      }

      &.widget-in-use {
        position: relative;

        &::after {
          content: '';
          position: absolute;
          left: 18px;
          z-index: 1;
          display: block;
          width: 10px;
          height: 10px;
          border-radius: 50%;
          background: @color-primary-color;
        }
      }

      label {
        display: -webkit-box;
        -webkit-line-clamp: 2;
        -webkit-box-orient: vertical;
        overflow: hidden;
        text-overflow: ellipsis;
        margin: 0px 5px;
        text-align: center;
        cursor: inherit;
        pointer-events: none;
      }

      .item-icon {
        background-color: #b5b5b5;
        width: 65px;
        min-width: 65px;
        border-radius: 4px;
        height: 25px;
        color: #3f3f40;
        display: flex;
        justify-content: center;
        align-items: center;
        box-shadow: #383536 0px 1px 5px;
        font-size: 11px;
        text-align: center;
        pointer-events: none;

        img {
          width: 20px;
          filter: invert(1);
          opacity: 0.7;
        }
      }

      &.dragging {
        width: 100%;
        background-color: @color-navigation-background;        
        border-color: #5abaf9;
      }

      &.disabled {
        background-color: #242424;
      }
    }
  }

  .layout-widgets {
    display: flex;
    flex-wrap: wrap;
    height: calc(100% - 35px);
    overflow: auto;

    & > div {
      width: 50%;
    }    
  }
}

#app.tabs-open #tabs-drawer-layout {
  left: 0;
}

nav {
  background-color: @color-navigation-background;
}

#panels {
  min-height: 50px;
  grid-column-start: 2;
  grid-row-start: 2;

  display: grid;
  grid-template-columns: 50% 50%;
  grid-template-rows: 50% 50%;
  width: 100%;
  height: calc(100% - @top-bar-height);

  align-items: flex-start;
  transition: all 0.2s ease;

  &.layout-CustomActor {
    grid-template-columns: 100%;
    grid-template-rows: 100%;
  }

  &.layout-Screen,
  &.layout-Stack,
  &.layout-Empty {
    grid-template-columns: 100%;
    grid-template-rows: 100%;
  }

  &.layout-1x1 {
    grid-template-columns: 100%;
    grid-template-rows: 100%;
  }

  &.layout-1x2 {
    grid-template-columns: 100%;
    grid-template-rows: 50% 50%;
  }

  &.layout-2x1 {
    grid-template-columns: 50% 50%;
    grid-template-rows: 100%;
  }


  .panel {
    height: 100%;
    width: 100%;
    background-color: @tabs-content-background-color;
    box-shadow: 0 3px 6px rgba(0,0,0,0.16), 0 3px 6px rgba(0,0,0,0.23);
    border-radius: @panel-border-radius;
    overflow-x: hidden;
    font-size: 14px;
    display: flex;
    justify-content: center;
  }
}

.fullscreen {
  position: fixed;
  top: 0px;
  left: 0px;
  z-index: 99999;
  display: flex;
  flex-direction: column;
  justify-content: center;
  align-items: center;
  width: 100%;
  height: 100%;
  background-color: @fullscreen-background-color;

  .app-icon {
    margin: 20px;
    filter: invert(100%) brightness(104%);
    opacity: 1;
    color: #555;
    font-size: 100px;

    .logo {
      animation: loadingIcon 2s ease-in-out infinite;
    }
  }

  &.login-screen {
    background-color: @loginscreen-background-color;
    flex-direction: row;

    .hint {
      font-size: 12px;
      color: @btn-secondary-background-color;
      margin-top: 5px;
    }

    .breadcrumb {
      font-style: italic;
    }
    .form {
      flex-direction: column;
      align-items: flex-start;
      justify-content: center;
      display: flex;
      font-size: 24px;
      padding-left: 50px;
      border-left: solid 1px #303030;
      height: 240px;

      input {
        margin-top: 20px;
        font-size: 24px;
      }
    }

    .icon-wrapper {
      text-align: center;
      padding-right: 50px;
    }

    .login-status {
      font-size: 16px;
      margin-top: 20px;
      color: @error-text-color;
      height: 18px;
    }
    .logo {
      animation: none;
    }
  }
}

.opaque {
  margin: 0;
  background-color: @color-primary-background; 
}

.dropdown {
  position: relative;

  .dropdown-menu {
    position: absolute;
    right: 0px;
    max-height: 0px;
    list-style: none;
    padding-inline-start: 0px;
    width: fit-content;
    transition: all 0.5s ease-in-out;
    overflow: hidden;
    background-color: @dropdown-menu-background;
    box-shadow: none;

    li {
      white-space: nowrap;
      padding: 10px;
    }
  }

  &:hover {
    .dropdown-menu {
      max-height: 200px;
      box-shadow: darken(@dropdown-menu-background, 10%) 0px 2px 5px 1px;
    }
  }
}

#app.tabs-open {
  .customstack-list {
    max-width: 1145px;
  }

  #tabs-drawer-layout {
    @media only screen and (max-device-width: 1366px) {
      width: @panel-content-with-tablet;
    }
  }

  #panels {
    width: calc(100% - 400px);
    margin: 0 0 0 400px;

    @media only screen and (max-device-width: 1366px) {
      width: calc(100% - @panel-content-with-tablet);
      margin: 0 0 0 @panel-content-with-tablet;
    }

    .actions {
      opacity: 1;
      pointer-events: unset;
    }
  }

}


input {
  -webkit-user-select: text;
  background-color: @input-field-background-color;
  border: solid 1px darken(@input-field-background-color, 2);
  color: @input-field-color;
  padding: 10px;

  &:focus {
    outline: none;
  }
}

.btn {
  border: solid 1px #000;
  border-radius: @btn-border-radius;
  padding: @btn-padding;
  margin: 0px 10px;

  &:focus {
    outline: none;
  }
}

.btn.btn-primary.disabled,
.btn.btn-secondary.disabled,
.btn.btn-danger.disabled,
.btn.disabled {
  background-color: darken(@btn-secondary-background-color, 2);
  border-color: darken(@btn-secondary-background-color, 20);
  color: darken(@btn-secondary-color, 35);
  pointer-events: none;
}

.btn.btn-primary {
  border: solid 1px @btn-primary-border-color;
  background-color: @btn-primary-background-color;
  color: @btn-primary-color;
  transition: all 0.2s ease;

  &:hover {
    background-color: darken(@btn-primary-background-color, 2);
    cursor: pointer;
  }
}

.btn.btn-secondary {
  border: solid 1px @btn-secondary-border-color;
  background-color: @btn-secondary-background-color;
  
  color: @btn-secondary-color;
  transition: all 0.2s ease;

  &:hover {
    background-color: darken(@btn-secondary-background-color, 2);
    cursor: pointer;
  }
}

.btn.btn-danger {
  border: solid 1px @btn-danger-border-color;
  background-color: @btn-danger-background-color;
  
  color: @btn-danger-color;
  transition: all 0.2s ease;

  &:hover {
    background-color: darken(@btn-danger-background-color, 2);
    cursor: pointer;
  }
}

.btn.selected {
  background-color: @selected-background-color;
  color: #FFF;

  &:hover {
    background-color: @selected-background-color;
    cursor: pointer;
  }
}

.btn-group {  
  .btn {
    margin: 0px;
    border-right: 0px;
    border-radius: 0px;

    &:first-child {
      border-top-left-radius: 4px;
      border-bottom-left-radius: 4px;
    }
    &:last-child {
      border-top-right-radius: 4px;
      border-bottom-right-radius: 4px;
      border-right: solid 1px #000;
    }
  }
}

.widgets-tab {
  height: 100%;
}

.search-wrapper {
  position: relative;

  .fa-search {
    position: absolute;
    outline: solid 0px red;
    top: 10px;
    padding: 0px 10px;
    left: 6px;
  }

  .fa-times-circle {
    position: absolute;
    outline: solid 0px red;
    height: 100%;
    padding: 0px 10px;
    right: 0px;
  }

  .list-search {
    width: 100%;
    border: 0px;
    padding-left: 40px;
    background-color: @tree-search-background;
    box-sizing: border-box;
  }
}

input:focus {
  outline: 0px;
}

#tabs-drawer-layout {
  .switch {
    &.toggle-mode {
      width: 120px;

      &:before {
        content: "Tree";
      }

      &:after {
        content: "Edit Tab";
      }
    }

    input:checked + .slider.inline:before {
      width: 60px;
      transform: translateX(50px);
    }
  }
}

.button-list {
  display: flex;
  flex-direction: column;
  height: 100%;
  margin: 10px 0 0;
  overflow-y: auto;

  .btn {
    margin: 10px;
  }
}

.safe-image-wrapper {
  width: 100px;
  height: 70px;
  display: flex;
  flex-direction: column;
  align-items: center;
  justify-content: center;

  svg {
    min-width: 40px;
    height: 40px;
  }
}

.dropwdown-row {
  .title {
    max-height: 32px;
  }
}

.dropdown-widget {
  position: relative;
  flex-grow: 1;

  .unbound-icon {
    position: absolute;
    top: 0;
    bottom: 0;
    left: 115px;
    z-index: 1;
    margin: auto 0;
    color: #e2ee2d;
  }

  .dropdown {
    width: 100%;
    padding: 0 20px 0 10px;
    height: 25px;
    font-size: 15px;
    border-radius: 4px;
    border-color: @widget-dropdown-border;
    background-color: @widget-dropdown-background;
    color: @widget-dropdown-color;

    &:focus-visible {
      outline: none;
    }
  }

  select, option {
    -webkit-appearance: none;  
    background-color: #202020;
  }

  &::after {
    content: '\0025BC';
    font: normal normal normal 12px/1 FontAwesome;
    color: white;
    right: 10px;
    top: 0px;
    height: 27px;
    padding: 0px 4px;
    position: absolute;
    pointer-events: none;
    outline: solid 0px red;
    display: flex;
    align-items: center;
  }
}

#tabs-drawer-layout {
  .dropdown-widget {
    .dropdown {
      height: 37px;
      padding: 0 30px 0 10px;
      border: solid 1px #141414;
      background-color: #242424;
      border-radius: 0;
    }
    &::after {
      height: 37px;
      right: 15px;
    }
  }
}

.grip-handle {
  width: 15px;
  height: 18px;
}

.presets-tab {
  height: 100%;

  .presets-wrapper {
    height: calc(100% - 35px);
    overflow-y: auto;
  }

  .preset-btn {
    background-color: @presets-btn-background;
    border: @preset-btn-border;
    border-radius: @preset-btn-border-radius;
    display: flex;
    min-height: 38px;
    padding: 0px 18px;
    padding-left: 15px;
    align-items: center;
    margin: 10px;
    position: relative;
    width: calc(100% - 55px);
    color: @preset-btn-color;
    cursor: pointer;

    .spin-icon {
      visibility: hidden;

      &.visible {
        visibility: visible;
      }
    }

    .favorite-icon {
      padding: 12px;
      
      cursor: pointer;
    }

    .item-icon {
      margin-left: auto;
    }

    &.active {
      color: @preset-btn-active-color;
      background-color: @preset-btn-active-background;
      cursor: unset;

      .item-icon {
        display: none;
      }
    }
  }
}