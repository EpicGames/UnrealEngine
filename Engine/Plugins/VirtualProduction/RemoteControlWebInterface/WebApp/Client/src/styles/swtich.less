.toggle-mode {
  border: solid 0px red;

  input[type="checkbox"] {
    cursor: pointer;
    position: relative;
    display: inline-block;
    -webkit-appearance: none;
    -webkit-tap-highlight-color: transparent;
    height: 30px;
    width: 110px;
    border-radius: 1.5em;
    background-color: #000;
    border-color: transparent;
    background-clip: padding-box;
    color: #e2e5e5;
    vertical-align: middle;

    &::before {
      content: "";
      position: absolute;
      top: 0px;
      left: 1px;
      bottom: 0;
      background-color: white;
      border-radius: 100%;
      border: 0.125em solid transparent;
      background-clip: padding-box;
      transform-origin: right center;
      transform: translateZ(0);
      backface-visibility: hidden;
      z-index: 2;
      width: 23px;
      height: 23px;
      transition: all 0.2s ease;
    }

    &::after {
      position: absolute;
      left: 29px;
      top: 2px;
      line-height: 2;
      font-weight: bold;
      font-size: 12px;
      text-transform: uppercase;
      content: "CONTROL";
      letter-spacing: 0;
      word-spacing: 5px;
      z-index: 1;
    }

    &:focus {
      color: white;
      border-color: transparent;
      outline: none;
    }

    &:checked {
      color: white;
      background-color: @color-primary-color;
      border-color: transparent;

      &:after {
        content: "DESIGN";
      }

      &::before {
        transform-origin: left center;
        left: calc(100% - 27px);
      }
    }
  }
}

.locked-mode {
  input[type="checkbox"] {
    width: 75px;
    &::after {
      font-family: 'Font Awesome 5 Free';
      font-weight: 900;
      content: "\f3c1";
      
      
    }

    &:checked {
      &::after {
        content: "\f023";
        
      }
      &:checked {
        color: white;
        background-color: red;
      }
    }
  }
}

.color-mode-toggle,
.slider-mode-toggle,
.boolean-toggle {
  position: relative;
  font-size: 10px;

  .on { color: #cdcccc; font-weight: normal; }
  .off { color: #FFF; font-weight: bold; }

  &.checked {
    .on { color: #FFF; font-weight: bold; }
    .off { color: #cdcccc; font-weight:  normal; }
  }

  input[type="checkbox"] {
    width: 140px;
    background-color: #171717;
    &::before {
      background-color: #242424;
      width: 65px;
      height: 24px;
      border-radius: 30px;
    }
    &::after {
      content: "";    
    }

    &:checked {
      &::before {
        background-color: #242424;
        width: 65px;
        border-radius: 30px;
        left: calc(100% - 69px);
      }
      &::after {
        content: "";
        
      }
      &:checked {
        background-color: #171717;
      }
    }
  }

  .labels {
    position: absolute;
    top: 4px;
    left: 0px;
    display: flex;
    z-index: 10;
    width: 95px;
    justify-content: space-between;
    height: 30px;
    align-items: center;
    padding: 0px 25px;
    pointer-events: none;
  }
}


.boolean-toggle {
  input[type="checkbox"] {
    width: 90px;

    &::before {
      width: 40px;
    }

    &:checked {
      &:before {
        width: 40px;
        left: calc(100% - 44px);
      }
    }
  }

  .labels {
    width: 60px;
    padding: 0px 20px;
  }
}

.UseClockSystem-toggle {
  margin-bottom: 5px;
}

.useCCmult-toggle,
.UseClockSystem-toggle,
.Power-Window-Mode-toggle,
.StickToWall-toggle,
.bShowGuides-toggle,
.Invert-toggle,
.Enabled-toggle {
  .boolean-toggle:not(.checked) {
    .labels {
      .off {
        color: #3C89F7;
      }
    }

    input[type="checkbox"]:not(:checked){
     
      &::before {
        border: solid 2px #3C89F7;
      }
    }
  }
  
}

@toggleWidth: 207px;
@toggleHeight: 44px;
.useCCmult-toggle,
.UseClockSystem-toggle,
.Power-Window-Mode-toggle,
.StickToWall-toggle,
.bShowGuides-toggle,
.Invert-toggle,
.Enabled-toggle {
  input[type="checkbox"]{
    height: @toggleHeight;
    border-radius: 3px;
    width: @toggleWidth;

    &::before {
      top: 4px;
      height: calc(@toggleHeight - 15px);
      border-radius: 5px;
      font-size: 20px;
      width: 100px;
      font-size: 18px;
      
      border: solid 2px #242424;
      //border: solid 2px red;

    }
  }

  input[type="checkbox"]:checked{

    &::before {
      top: 4px;
      height: calc(@toggleHeight - 15px);
      border-radius: 5px;
      font-size: 18px;
      width: 100px;
      left: calc(100% - 110px);
    }
  }

  .labels {
    width: 100%;
    height: @toggleHeight;
    padding: 0px 40px;
    display: flex;
    justify-content: space-between;
    align-items: center;
    font-size: 18px;
    outline: solid 0px red;
    box-sizing: border-box;
    /*
    padding: 5px 25px;
    font-size: 20px;
   
    */
  }
}