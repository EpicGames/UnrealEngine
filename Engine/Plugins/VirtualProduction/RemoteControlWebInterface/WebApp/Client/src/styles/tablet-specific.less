/* 
  ##Device = Tablets, Ipads (landscape)
  ##Screen = B/w 768px to 1024px
*/

@media (min-device-width: 768px) and (max-device-width: 1024px)  {
  
  // COLOR PICKERS
  .navigation-list-panels {
    .color-picker {
      &-container {
        width: 260px;
      }
    }
  }

  // WIDGET DRAWER
  .items-list, .layout-widgets {
    .item-label {
      display: none;
    }
  }
  
}

.ipad-debug {
  position: fixed;
  right:  10px;
  top: 40px;
  background-color: blue;
  border-radius: 50%;
  width: 5px;
  height: 5px;
  z-index: 999;
}

//iPad 9.7"
@media screen and (min-width: 1024px) {
  .color-correction-wrapper {
    max-height: 900px;

    .ipad-debug {
      background-color: green;
    }
    
    .ref {
      display: none;
    }

    .mode-button {
      height: 60px;
      width: 40px;
    }
    .grid-row {
      height: 90px;
      margin: 10px 0;
    }
    .color-picker-singleton {
      @color-picker-size: 210px;
      
      .color-wheel {
        width: @color-picker-size + 50px;
        margin-top: 0px;
      }
      
      .rectangle {
        width: 205px;
        height: 215px;
      }

      .color-picker-slider-wheel.vertical {
        min-height: 210px;
        max-height: 210px;
        margin-top: 27px;
      }
      .color-picker-container {
        width: @color-picker-size;
        height: @color-picker-size;
        margin-top: 14px;
        margin-left: 29px;
      
        .color-picker-wheel {
          width: @color-picker-size;
          .circle {
            width: @color-picker-size - 20px;
            min-height: @color-picker-size - 20px;
            .circle-gradient {
              width: @color-picker-size - 20px;
              height: @color-picker-size - 20px;
              &:after {
                width: @color-picker-size - 20px;
                height: @color-picker-size - 20px;
              }
              .canvas-area {
                width: @color-picker-size - 30px;
                height: @color-picker-size - 30px;
              }
            }
          }
  
        }
      }

      .color-property-selection {
        margin-top: 0px;
        max-width: unset;
        font-size: 14px;
      }

      .details-section .rgb-sliders .slider-wheel-container {
        margin: 12px 0px;
        outline: solid 0px yellow;
      }
    }

    .cc-group.cc-panel {
      min-height: 300px;
    }

    .shotbox {
      min-width: 410px;

      .light-card {
        width: 62px;
        height: 63px;
        font-size: 10px;
      }
    }

    .control-rows {
      > .control {
        margin-top: 5px;

        &.wrap > label {
          top: -8px;
        }
      }
    }
  }
}

// IPAD PRO 10.5"
@media screen and (min-width: 1112px)   {
  .color-correction-wrapper {

    .ipad-debug {
      background-color: red;
    }

    .ref {
      display: none;
    }

    .mode-button {
      height: 60px;
      width: 40px;
    }
    .grid-row {
      height: 90px;
      margin: 10px 0;
    }
    .color-picker-singleton {
      @color-picker-size: 210px;
      
      .color-wheel {
        width: @color-picker-size + 50px;
        margin-top: 0px;
      }
      
      .color-picker-slider-wheel.vertical {
        min-height: @color-picker-size;
        max-height: @color-picker-size;
        margin-top: 27px;
      }
      .color-picker-container {
        width: @color-picker-size;
        height: @color-picker-size;
        margin-top: 14px;
        margin-left: 29px;
      
        .color-picker-wheel {
          width: @color-picker-size;
          .circle {
            width: @color-picker-size - 20px;
            min-height: @color-picker-size - 20px;
            .circle-gradient {
              width: @color-picker-size - 20px;
              height: @color-picker-size - 20px;
              &:after {
                width: @color-picker-size - 20px;
                height: @color-picker-size - 20px;
              }
              .canvas-area {
                width: @color-picker-size - 30px;
                height: @color-picker-size - 30px;
              }
            }
          }
  
        }
      }

      .color-property-selection {
        margin-top: 0px;
        max-width: unset;
        font-size: 14px;
      }

      .details-section .rgb-sliders .slider-wheel-container {
        margin: 12px 0px;
        outline: solid 0px yellow;
      }
    }

    .cc-group.cc-panel {
      min-height: 380px;
    }
    .shotbox {
      min-width: 410px;

      .light-card {
        width: 62px;
        height: 80px;
        font-size: 10px;
      }
    }

    .control-rows {
      > .control {
        margin-top: 10px;

        &.wrap > label {
          top: -8px;
        }
      }
    }
  }
}



// IPAD PRO 12.9"
@media screen and (min-width: 1366px) { 
  .color-correction-wrapper {
    max-height: 1400px;

    .ipad-debug {
      background-color: orange;
    }

    .rgb -sliders {
      display: flex;
      flex-direction: column;
      justify-content: space-evenly;
      max-height: 416px;
      max-height: 416px;
    }

    .shotbox {
      min-width: 560px;

      .light-card {
        width: calc(212px / 2);
        height: calc(160px / 2);
        font-size: 14px;
      }
    }


    .color-picker-singleton {
      @color-picker-size: 330px;
      
      .color-wheel {
        width: @color-picker-size + 50px;
        margin-top: 0px;
      }

      .color-picker-slider-wheel.vertical {
        min-height: @color-picker-size;
        max-height: @color-picker-size;
        margin-top: 27px;
      }
      .color-picker-container {
        width: @color-picker-size;
        height: @color-picker-size;
        min-height: @color-picker-size;
        margin-top: 14px;
        margin-left: 29px;
      
        .color-picker-wheel {
          width: @color-picker-size;
          .circle {
            width: @color-picker-size - 20px;
            min-height: @color-picker-size - 20px;
            .circle-gradient {
              width: @color-picker-size - 20px;
              height: @color-picker-size - 20px;
              &:after {
                width: @color-picker-size - 20px;
                height: @color-picker-size - 20px;
              }
              .canvas-area {
                width: @color-picker-size - 30px;
                height: @color-picker-size - 30px;
              }
            }
          }
          .rectangle {
            width: 300px;
            height: 310px;
            border: solid 0px red;
          }
  
        }
      }

      .color-property-selection {
        margin-top: 0px;
        max-width: unset;
        font-size: 14px;
      }

      .details-section .rgb-sliders .slider-wheel-container {
        margin: 20px 0px;
        outline: solid 0px yellow;
      }
    }

  }
}
