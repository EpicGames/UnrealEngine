@top-bar-height: 32px;
@drawer-width: 50px;

@scrollbar-track-color: #585c65;
@scrollbar-thumb-color: #000;

@color-primary-background: #353944;
@color-navigation-background: #242424;
@color-primary-color: #5abaf9;
@fullscreen-background-color: rgba(0,0,0,0.5);
@loginscreen-background-color: rgba(0,0,0,1);

@color-primary-text: #cdcccc;

@font-primary-family: Arial, Helvetica, sans-serif;

@input-field-background-color: transparent; //#191919;
@input-field-border-color: #767676;
@input-field-border-size: 1px;
@input-field-border-radius: 0px; //5px;

@tabs-background-color: #242424;
@tabs-selected-color: #161616;
@tabs-text-color: #cdcccc;
@tabs-selected-text-color: #5abaf9;
@tabs-content-background-color: #161616;

@panel-background-color: #202020;
@panel-title-background-color: #171717;
@panel-border-radius: 0px;

@dropdown-menu-background: #282b34;

@widget-title-height: 40px;

@widget-tab-content-background: #171717;

@widget-dial-background-wrapper-color: #161616;
@widget-dial-background-color: #202020;
@widget-dial-border-color: #2c2c2c;
@widget-dial-shadow-color: #00000066;
@widget-dial-needle-color: #0067b4;
@widget-dial-tick-large: #ffffffa6;
@widget-dial-tick-short: #ffffff12;
@widget-dial-value-color: #ffffffa6;

@widget-joystick-background-color: #161616;
@widget-joystick-foreground-color: #606060;
@widget-joystick-label-color: #393939;
@widget-joystick-value-height: 25px;
@widget-joystick-border-width: 2px;
@widget-joystick-inner-border: solid @widget-joystick-border-width #7d7d7d;
@widget-joystick-outer-border: solid @widget-joystick-border-width #2d2d2d;
@widget-joystick-inner-circle-color: #1A1A1A;
@widget-joystick-inner-circle-border: solid @widget-joystick-border-width #2e2e2e;
@widget-joystick-arrow-color: #606060;

@widget-slider-track-color: #323232;
@widget-slider-track-active-color: #3C89F7;
@widget-slider-knob-color: #3C89F7;
@widget-slider-value-background-color: #20272f;
@widget-slider-value-height: 30px;
@widget-slider-track-height: 2px;
@widget-slider-thumb-size: 15px;

@widget-dropdown-background: @input-field-background-color;
@widget-dropdown-border: #000;
@widget-dropdown-color: #FFF;

@widget-button-background: #181818;
@widget-button-background-notactive: #242424;
@widget-button-shadow: 0 1px 0 rgb(76 76 76 / 50%) inset, 0 -1px 0 rgb(255 255 255 / 10%) inset, 0 4px 0 #0a0a0a, 0 4px 2px rgb(0 0 0 / 50%);


@btn-widget-border-color: #000;
@btn-widget-background-color: #242730;
@btn-widget-color: #FFF;

@presets-selected-background-color: #5abaf9;

@modal-background-color: #242424;


@lists-border-color: #282b34;
@lists-item-highlight-background-color: #303030;
@lists-item-selected-background-color: #5abaf9;
@lists-item-selected-color: #FFF;


@input-field-color: #FFF;

@btn-border-radius: 4px;
@btn-padding: 10px 15px;

@btn-primary-border-color: #000;
@btn-primary-background-color: #5abaf9;
@btn-primary-color: #FFF;

@btn-primary-selected-border-color: #000;
@btn-primary-selected-background-color: #5abaf9;
@btn-primary-selected-color: #fff;

@btn-secondary-border-color: #000;
@btn-secondary-background-color: #5a5a5a;
@btn-secondary-color: #FFF;

@btn-danger-border-color: darken(#f44336, 20);
@btn-danger-background-color: #f44336;
@btn-danger-color: #FFF;

@text-color: #FFF;
@error-text-color: #f44336;
@background-button-color: #222;
@border-color: #555;
@primary-border: solid 1px @border-color;
@selected-background-color: @presets-selected-background-color;
@secondary-selected-color: #5abaf9;

@actor-title-width: 110px;
@navigation-btn-width: 150px;
@navigation-width: @navigation-btn-width;

@presets-btn-background: #484546;
@preset-btn-border: solid 1px #565656;
@preset-btn-border-radius: 4px;
@preset-btn-color: #cdcccc;
@preset-btn-active-color: #FFF;
@preset-btn-active-background: @color-primary-color;


@tree-search-background: @input-field-background-color;


@panel-content-with-tablet: 280px;

@sequencer-color: #c85050;

@playlist-border-color: #161616;
@playlist-active-stop-color: #ff4040;
@playlist-active-play-color: #8bc24a;
@playlist-active-rehold-color: #5abaf9;