#top-bar {
  height: @top-bar-height;
  width: 100%;
  grid-column: 1 / span 3;
  display: flex;
  align-items: center;
  font-size: 12px;

  .toggle-mode input[type="checkbox"]::before {
    width: 18px;
    height: 18px;
    top: 4px;
  }

  .app-icon {
    width: @drawer-width;
    filter: invert(100%) sepia(99%) saturate(0%) hue-rotate(15deg) brightness(104%) contrast(100%);
    margin: 0px 1px;
    display: flex;
    justify-content: center;
    align-items: center;

    img {
      width: 22px;
    }
  }

  .tabs-wrapper {
    display: flex;
    width: 100%;
    height: 100%;
    position: relative;
    overflow: hidden;
    padding-left: 1rem;
    
    .tabs-wrapper-inner {
      width: calc(100% - 300px);

      .tabs-drop-list {
        display: flex;
        width: 100%;
        overflow: auto;
        position: relative;
      }
    }

    .tab {
      display: flex;
      align-items: center;
      padding: 0px 20px 0px 20px;
      background-color: @tabs-background-color;
      height: @top-bar-height;
      color: @tabs-text-color;
      min-width: 90px;
      max-width: 150px;
      flex: none;
      white-space: nowrap;
      overflow: hidden;
      text-overflow: ellipsis;
      font-size: 14px;
      cursor: pointer;
      position: relative;
      
      .icon {
        margin: 0px 10px 0px 0px ;
      }
      &.selected {
        background-color: @tabs-selected-color;
        color: @tabs-selected-text-color;
      }

      &.edit-tab,
      &.new-tab { 
        min-width: 100px;
        width: 100px;
        max-width: 100px;
        padding: 0px;
        display: flex;
        justify-content: center;
        cursor: pointer;
      }

      &.locked-toggle {
        min-width: 75px;
        width: 75px;
        max-width: 75px;
      }
      &.new-tab {
        min-width: 40px;
        width: 40px;
        max-width: 40px;
      }

      .delete-tab {
        outline: solid 0px red;
        padding: 0px 10px 0px 20px;
        height: 100%;
        display: flex;
        justify-content: center;
        align-items: center;
      }

      .label {
        white-space: nowrap;
        overflow: hidden;
        text-overflow: ellipsis;
      }
    }

    .tab-navigation {
      position: absolute;
      right: 0px;
      height: 100%;
      display: flex;
      align-items: center;
      width: 300px;
      padding-right: 15px;
      justify-content: flex-start;
      background-color: @tabs-background-color;
      
      min-width: 15px;
      min-height: 15px;

      .arrow {
        width: 40px;
        height: 100%;
        display: flex;
        justify-content: center;
        align-items: center;
        cursor: pointer;

        &.disabled {
          color: darken(@tabs-text-color, 50%);
          cursor: unset;
        }
      }

      .preset-name {
        overflow: hidden;
        flex-grow: 1;
        max-width: 220px;
        text-align: center;
        text-overflow: ellipsis;
      }
      
      .preset-undo-redo {
        min-width: 60px;

        .icon {
          padding: 8px;
          cursor: pointer;

          &.disabled {
            opacity: .3;
            pointer-events: none;
          }
        }
      }
    }
  }
}

