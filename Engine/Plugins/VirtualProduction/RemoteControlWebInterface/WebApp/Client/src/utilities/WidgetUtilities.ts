import { ColorProperty, IPreset, PropertyType, VectorProperty, WidgetType, WidgetTypes, IHsvColor } from "src/shared";


type Range = {
  min?: number;
  max?: number;
}

export type WidgetProperties<TPropertyValue = any> = {
  stack?: boolean;
  type?: PropertyType;
  value?: TPropertyValue;
  onChange?: (value: TPropertyValue) => void;
};

type CustomWidget = {
  type: string;
  render: (props: any) => React.ReactNode;
};

export class WidgetUtilities {

  static propertyWidgets: Record<string, CustomWidget[]> = {};
  static customWidgets: Record<string, CustomWidget> = {};
  
  static getPropertyLimits(type: PropertyType): Range {
    switch (type) {
      case PropertyType.Int8:
        return { min: -128, max: 127 };

      case PropertyType.Int16:
        return { min: -32768, max: 32767 };

      case PropertyType.Int32:
        return { min: -2147483648, max: 2147483647 };

      case PropertyType.Uint16:
        return { min: 0, max: 65535 };

      case PropertyType.Uint32:
        return { min: 0, max: 4294967295 };

      case PropertyType.Float:
      case PropertyType.Vector:
      case PropertyType.Vector2D:
      case PropertyType.Vector4:
      case PropertyType.LinearColor:
        return { min: -1.17549e+38, max: 3.402823466E+38 };

      case PropertyType.Rotator:
        return { min: 0, max: 360	};

      case PropertyType.Double:
        return { min: Number.MIN_VALUE, max: Number.MAX_VALUE };

      case PropertyType.Color:
        return { min: 0, max: 255 };
    }

    return {};
  }

  static parseNumber(value: string): number {
    const number = parseFloat(value);
    if (!isNaN(number))
      return number;
  }

  static getMinMax(preset: IPreset, exposed: string): Range {
    const property = preset.Exposed[exposed];

    return {
      min: WidgetUtilities.parseNumber(property?.Metadata?.Min),
      max: WidgetUtilities.parseNumber(property?.Metadata?.Max),
    };
  }

  static registerWidget(type: string, properties: PropertyType[], render: (props: WidgetProperties) => React.ReactNode): void {
    const custom: CustomWidget = { type, render };
    WidgetUtilities.customWidgets[type] = custom;
    
    for (const property of properties) {
      if (!WidgetUtilities.propertyWidgets[property])
        WidgetUtilities.propertyWidgets[property] = [];

      WidgetUtilities.propertyWidgets[property].push(custom);
    }
  }

  static getPropertyPrecision(propertyType: PropertyType) {
    switch (propertyType) {
      case PropertyType.Float:
      case PropertyType.Double:
      case PropertyType.Vector:
      case PropertyType.Vector2D:
      case PropertyType.Vector4:
      case PropertyType.LinearColor:
        return 3;

      case PropertyType.Rotator:
        return 2;
    }

    return 0;
  }

  static getPropertyKeys(propertyType: PropertyType) {
    switch (propertyType) {
      case PropertyType.Rotator:
        return ['Roll', 'Pitch', 'Yaw'];

      case PropertyType.Vector:
        return ['X', 'Y', 'Z'];

      case PropertyType.Vector2D:
        return ['X', 'Y'];

      case PropertyType.Vector4:
        return ['X', 'Y', 'Z', 'W'];

      case PropertyType.Color:
      case PropertyType.LinearColor:
        return ['R', 'G', 'B'];
    }

    return [];
  };

  static getCompatibleWidgets(propertyType: PropertyType): WidgetType[] {
    switch (propertyType) {
      case PropertyType.Int16:
      case PropertyType.Int32:
      case PropertyType.Int64:
      case PropertyType.Uint16:
      case PropertyType.Uint32:
      case PropertyType.Uint64:
      case PropertyType.Float:
      case PropertyType.Double:
        return [WidgetTypes.Slider, WidgetTypes.Dial];
      
      case PropertyType.Boolean:
      case PropertyType.Uint8:
        return [WidgetTypes.Toggle];

      case PropertyType.Vector:
      case PropertyType.Vector2D:
        return [WidgetTypes.Vector, WidgetTypes.Joystick, WidgetTypes.Sliders, WidgetTypes.Dials];

      case PropertyType.Vector4:
      case PropertyType.LinearColor:
      case PropertyType.Color:
        return [WidgetTypes.ColorPicker, WidgetTypes.MiniColorPicker];

      case PropertyType.Rotator:
        return [WidgetTypes.Vector, WidgetTypes.Sliders, WidgetTypes.Dials];

      case PropertyType.String:
        return [WidgetTypes.Text];
    }

    if (propertyType && (propertyType.startsWith('TEnum') || propertyType.startsWith('E')))
      return [WidgetTypes.Dropdown];

    // Unknown Property, show as a Asset
    return [WidgetTypes.Asset];
  }

  static colorToRgb(color: ColorProperty | VectorProperty, max: number) {
    const rgb = color as ColorProperty;
    const vector = color as VectorProperty;

    max = max || 1;
    const rgbColor = {
      R: (vector?.X ?? rgb?.R) * 255 / max,
      G: (vector?.Y ?? rgb?.G) * 255 / max,
      B: (vector?.Z ?? rgb?.B) * 255 / max,
    } as ColorProperty;

    if (rgb?.A)
      rgbColor.A = rgb.A;

    return rgbColor;
  }

  static rgbToColor = (rgb: ColorProperty, max: number, vector = true): ColorProperty | VectorProperty => {
    const a = rgb.R / 255 * max;
    const b = rgb.G / 255 * max;
    const c = rgb.B / 255 * max;

    if (vector)
      return { X: a, Y: b, Z: c };

    const color = { R: a, G: b, B: c } as ColorProperty;
    if (rgb.A)
      color.A = rgb.A;

    return color;
  }

  static rgb2Hsv(color: ColorProperty): IHsvColor {
    const r = color?.R / 255 ?? 1;
    const g = color?.G / 255 ?? 1;
    const b = color?.B / 255 ?? 1;

    const min = Math.min(r, g, b);
    const max = Math.max(r, g, b);

    let h = 0, s = 0, v = max;
    let d = max - min;
    s = max === 0 ? 0 : d / max;

    if (max !== min) {
      switch (max) {
        case r:
          h = (g - b) / d + (g < b ? 6 : 0);
          break;

        case g:
          h = (b - r) / d + 2;
          break;

        case b:
          h = (r - g) / d + 4;
          break;
      }

      h /= 6;
    }

    const hsvColor = { h, s, v } as IHsvColor;
    if (color?.A)
      hsvColor.a = color.A;

    return hsvColor;
  }

  static hsv2rgb({ h, s, v, a }: IHsvColor): ColorProperty {
    let r: number, g: number, b: number;

    let i = Math.floor(h * 6);
    let f = h * 6 - i;
    let p = v * (1 - s);
    let q = v * (1 - f * s);
    let t = v * (1 - (1 - f) * s);

    if (i % 6 === 0)
      ([r, g, b] = [v, t, p]);

    if (i % 6 === 1)
      ([r, g, b] = [q, v, p]);

    if (i % 6 === 2)
      ([r, g, b] = [p, v, t]);

    if (i % 6 === 3)
      ([r, g, b] = [p, q, v]);

    if (i % 6 === 4)
      ([r, g, b] = [t, p, v]);

    if (i % 6 === 5)
      ([r, g, b] = [v, p, q]);

    const color = { R: r * 255, G: g * 255, B: b * 255 } as ColorProperty;
    if (a)
      color.A = a;

    return color;
  }
}