{
  "compilerOptions": {
    "baseUrl": "./",
    "target": "es6",
    "allowJs": true,
    "skipLibCheck": true,
    "esModuleInterop": true,
    "allowSyntheticDefaultImports": true,
    "forceConsistentCasingInFileNames": true,
    "module": "esnext",
    "resolveJsonModule": true,
    "isolatedModules": true,
    "experimentalDecorators": true,
    "emitDecoratorMetadata": true,
    "moduleResolution": "node",
    "noEmit": true,
    "strict": false,
    "jsx": "react",
    "lib": [
      "dom",
      "dom.iterable",
      "esnext"
    ]
  },
  "include": [
    "src", "../Shared"
  ]
}
