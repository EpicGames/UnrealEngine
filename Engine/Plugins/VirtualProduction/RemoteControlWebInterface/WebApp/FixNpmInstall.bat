REM @echo off
attrib -r .\package.json
attrib -r .\package-lock.json
attrib -r .\Server\package.json
attrib -r .\Server\package-lock.json
attrib -r .\Client\package.json
attrib -r .\Client\package-lock.json