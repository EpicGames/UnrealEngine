const fs = require('fs-extra');
const path = require('path');
const exec = require('child_process').exec;
const util = require('util');
const package = require('../package.json');

const execute = util.promisify(exec);

async function build() {
  try {
    const root = path.resolve(__dirname, '..');

    const client = { root: path.join(root, 'Client') };
    client.build = path.join(client.root, 'build');
    
    const server = { root: path.join(root, 'Server') };
    server.public = path.join(server.root, 'public');
  
    await Promise.all([
      execute('npm run build', { cwd: client.root }),
      execute('npm run build', { cwd: server.root }),
    ])
    
    await fs.copy(client.build, server.public);
    await fs.writeJSON(path.join(server.public, 'version.json'), { date: new Date(), version: package.version });
  
  } catch (err) {
    let message = err.stdout;
    if (!message)
      message = err.message;
    if (!message)
      message = err;
    
    console.log('Build failed: ', message);
    process.exit(1);
  }
}

build();
