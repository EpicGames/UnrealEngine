{
  "name": "conductor-server",
  "scripts": {
    "build": "tslint -c tslint.json -p tsconfig.json && tsc",
    "dev:start": "tslint -c tslint.json -p tsconfig.json --force && node ./build/server --dev",
    "start": "tsc-watch --onSuccess \"npm run dev:start\"",
    "serve": "node ./build/server"
  },
  "dependencies": {
    "@overnightjs/core": "^1.7.4",
    "body-parser": "^1.19.0",
    "express": "^4.17.1",
    "lodash": "^4.17.20",
    "socket.io": "^2.3.0",
    "superagent": "^6.0.0",
    "ws": "^7.3.1"
  },
  "devDependencies": {
    "@types/express": "^4.17.7",
    "@types/fs-extra": "^9.0.1",
    "@types/lodash": "^4.14.160",
    "@types/node": "^14.6.0",
    "@types/socket.io": "^2.1.11",
    "@types/superagent": "^4.1.10",
    "@types/ws": "^7.2.6",
    "nodemon": "^2.0.4",
    "tsc-watch": "^4.2.9",
    "tslint": "^6.1.3",
    "typescript": "^4.0.2",
    "typescript-tslint-plugin": "^0.5.5"
  }
}
