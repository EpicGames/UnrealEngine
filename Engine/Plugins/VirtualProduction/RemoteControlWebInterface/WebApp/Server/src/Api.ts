import { Controller, Get, Post, Put, Delete } from '@overnightjs/core';
import { Request, Response } from 'express';
import { UnrealEngine } from './';


interface ISendOptions {
  type?: string;
  nullIsNotFound?: boolean;
}

@Controller('api')
export class Api {
  
  public async initialize(): Promise<void> {
  }

  @Get('connected')
  private connected(req: Request, res: Response) {
    this.send(res, Promise.resolve({ connected: UnrealEngine.isConnected() }));
  }

  @Get('presets')
  private presets(req: Request, res: Response) {
    this.send(res, UnrealEngine.getPresets());
  }

  @Get('passphrase')
  private async passphrase(req: Request, res: Response) {
	const passphrase = req.headers['passphrase']?.toString() ?? '';  
	const keyCorrect = await UnrealEngine.checkPassphrase(passphrase);
	if (!keyCorrect)
	  res.status(401);	
	
    this.send(res, Promise.resolve({ keyCorrect: keyCorrect}) );
  }

  @Get('payloads')
  private payloads(req: Request, res: Response) {
    this.send(res, UnrealEngine.getPayloads());
  }

  @Get('presets/:preset/load')
  private async load(req: Request, res: Response): Promise<void> {
    this.send(res, UnrealEngine.loadPreset(req.params.preset));
  }  

  @Get('presets/payload')
  private async payload(req: Request, res: Response): Promise<void> {
    this.send(res, UnrealEngine.getPayload(req.query.preset?.toString()));
  }

  @Get('presets/view')
  private view(req: Request, res: Response) {
    this.send(res, UnrealEngine.getView(req.query.preset?.toString()));
  }

  @Put('presets/:preset/favorite')
  private favorite(req: Request, res: Response) {
    this.send(res, UnrealEngine.favoritePreset(req.params.preset, !!req.body?.value));
  }

  @Put('proxy')
  private proxy(req: Request, res: Response) {
    this.send(res, UnrealEngine.proxy(req.body?.method, req.body?.url, req.body?.body, req.socket.remoteAddress.replace(/^.*:/, '')));
  }

  @Get('thumbnail')
  private thumbnail(req: Request, res: Response) {
    this.send(res, UnrealEngine.thumbnail(req.query.asset.toString(), req.socket.remoteAddress.replace(/^.*:/, '')));
  }

  @Get('shutdown')
  private shutdown(req: Request, res: Response) {
    res.send({ message: 'ok' });
    setImmediate(() => process.exit(0));
  }

  protected async send(res: Response, promise: Promise<any>, options?: ISendOptions): Promise<any> {
    try {

      const result = await promise;
      if (result === null && options?.nullIsNotFound)
        return res.status(404).send({ status: 'ERROR', error: 'Not found' });

      if (options?.type)
        res.set('Content-Type', options.type);

      res.send(result);

    } catch (err) {
      const error = err?.message ?? err;
      res.status(500)
          .send({ status: 'ERROR', error });
    }
  }
}
