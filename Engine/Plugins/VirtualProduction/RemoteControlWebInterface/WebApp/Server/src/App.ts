import { Server } from '@overnightjs/core';
import { Request, Response, NextFunction, static as expressStatic } from 'express';
import bodyParser from 'body-parser';
import path from 'path';
import request from 'superagent';
import { Program, Api, Notify, UnrealEngine, LogServer } from './';


export class App extends Server {

  constructor() {
    super();

    this.app.use(bodyParser.json());
    this.app.use(bodyParser.urlencoded({ extended: false }));
    this.app.use((req, res, next) => App.setAccessControl(res, next));
    this.app.use(expressStatic(path.join(Program.rootFolder, 'public'), { setHeaders: (res) => App.setAccessControl(res) }));
  }

  public async start(): Promise<void> {
    try {

      const api = new Api();
      api.initialize();
      super.addControllers([api]);
      this.app.use('/api', (req, res) => App.notFoundHandler(req, res, true));

      // Trying to kill a zombie process
      await request.get(`http://127.0.0.1:${Program.port}/api/shutdown`)
                    .timeout(100)
                    .then(() => new Promise(resolve => setTimeout(resolve, 100)))
                    .catch(() => {});

      await this.startServer();

      var hostname = Program.hostname;
      if (hostname == '0.0.0.0'){
        hostname = '127.0.0.1';
      }
      console.log(`DONE: WebApp started, ${hostname}:${Program.port}`, );

    } catch (error) {
      console.error('ERROR:', error.message);
      throw error;
    }
  }

  private startServer(): Promise<void> {
    return new Promise((resolve, reject) => {
      const server = this.app.listen(Program.port, Program.hostname,  async () => {
        await Notify.initialize(server);
        await UnrealEngine.initialize();
        await LogServer.initialize();
        resolve();
      });

      server.once('error', reject);
    });
  }

  private static setAccessControl(res: Response, next?: NextFunction): void {
    res.header('X-Powered-By', 'Conductor');
    res.header('Access-Control-Allow-Origin', '*');
    res.header('Access-Control-Allow-Headers', '*');
    res.header('Access-Control-Max-Age', '86400');
    res.header('Access-Control-Allow-Methods', 'GET, POST, PUT, DELETE, OPTIONS');

    next?.();
  }

  private static notFoundHandler(req: Request, res: Response, isApi: boolean): void {
    res.status(404)
      .send({ error: `Invalid ${isApi ? 'api call' : 'request'}: ${req.method} '${req.originalUrl}'` });
  }

}
