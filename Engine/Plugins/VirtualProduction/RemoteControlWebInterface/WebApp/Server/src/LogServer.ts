import ws from 'ws';
import _ from 'lodash';
import fs from 'fs-extra';
import { Program } from './';


// if command takes more than durationThreashold (in ms) log it in the "Highlights" file
const durationThreashold = 1000;

interface IRequestDuration {
  RequestId: number;
  payload: any;
  startTime: Date;
}

export namespace LogServer {

  var server: ws.Server;
  var loopback: ws;
  var buffer: string = '';
  var highlights: string = '';
  var durations: Record<number, IRequestDuration> = {};
  var startTime = new Date().toISOString().replace(/\:/g, '_');
  var flushTimer: NodeJS.Timeout;

  export async function initialize(): Promise<void> {
    const port = Program.dev ? 7002 : Program.port + 2;
    server = new ws.Server({ port });
    loopback = new ws(`ws://localhost:${port}`);

    await fs.ensureDir('./Logs');
    server.on('connection', (socket: ws) => {
      socket.on('message', onMessage);
    });

    flushTimer = setInterval(flushLogs, 30 * 1000);
  }

  export async function shutdown(): Promise<void> {
    server?.removeAllListeners();
    server?.close();
    server = null;
    clearInterval(flushTimer);
    flushTimer = null;
  }

  function onMessage(data: ws.Data) {
    const json = data.toString();
    try {
      const payload = JSON.parse(json);
      log(payload);
    } catch (err) {
      console.log("Error with message:", json);
      console.log(err);
    }
  }

  function log(payload: Record<string, any>, lastRequestLog?: boolean): void {
    const Timestamp = new Date().toISOString();
    buffer += JSON.stringify({ Timestamp, ...payload }) + '\r\n';

    const { RequestId } = payload;
    let info = durations[RequestId];
    if (!info) {
      info = { RequestId, payload, startTime: new Date() };
      durations[payload.RequestId] = info;
    }

    if (lastRequestLog) {
      const duration = new Date().valueOf() - info.startTime.valueOf();
      if (duration > durationThreashold) {
        highlights += `RequestId ${RequestId}, took ${duration}ms\r\n`;
        highlights += JSON.stringify(info.payload) + '\r\n\r\n';
      }

      delete durations[RequestId];
    }
  }

  export function logLoopback(payload: Record<string, any>, lastRequestLog?: boolean): void {
    if (loopback?.readyState === ws.OPEN)
      log(payload, lastRequestLog);
  }
  
  function flushLogs() {
    if (flushLogFile(buffer, 'all'))
      buffer = '';

    if (flushLogFile(highlights, 'highlights'))
      highlights = '';
  }

  function flushLogFile(content: string, type: string) {
    if (!content)
      return false;

    try {
      const filename = `./Logs/${startTime}-${type}.log`;
      fs.appendFileSync(filename, content, 'utf8');

      return true;
    } catch (err) {
      console.log('error', err);
    }

    return false;
  }
}
