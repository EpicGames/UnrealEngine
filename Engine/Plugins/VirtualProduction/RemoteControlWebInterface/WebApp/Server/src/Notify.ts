import { IView, PropertyValue } from '@Shared';
import { Server } from 'http';
import socketio from 'socket.io';
import { UnrealEngine } from './UnrealEngine';

                  
export namespace Notify {
  var io = socketio({ path: '/api/io' });

  export async function initialize(server: Server): Promise<void> {
    io.attach(server);

    io.sockets.on('connection', (socket: socketio.Socket) => {
      const ip = socket.handshake.address.replace(/^.*:/, '');
      socket
        .on('client', onPing)
        .on('view', (preset, view, supressUnrealNotification) => onViewChange(preset, view, supressUnrealNotification, ip))
        .on('value', (preset, property, value) => UnrealEngine.setPayloadValue(preset, property, value, ip))
        .on('execute', (preset, func, args) => UnrealEngine.executeFunction(preset, func, args, ip))
        .on('metadata', (preset, property, metadata, value) => UnrealEngine.setPresetPropertyMetadata(preset, property, metadata, value, ip))
        .on('rebind', (preset, properties, target) => UnrealEngine.rebindProperties(preset, properties, target, ip))
        .on('search', (query, types, prefix, filterArgs, count, callback) => UnrealEngine.search(query, types, prefix, filterArgs, count, callback, ip));

      if (UnrealEngine.isConnected())
        socket.emit('connected', true);
        
      socket.emit('opened', UnrealEngine.isOpenConnection());

      if (UnrealEngine.isLoading())
        socket.emit('loading', true);
    });
  }

  export function emit(what: 'presets' | 'payloads' | 'connected' | 'loading' | 'opened', ...args: any[]) {
    io.emit(what, ...args);
  }

  export function onViewChange(preset: string, view: IView, supressUnrealNotification: boolean, ip: string) {
    if (!supressUnrealNotification)
      UnrealEngine.setView(preset, view, ip);
    io.emit('view', preset, view);
  }

  export function emitValueChange(preset: string, property: string, value: PropertyValue) {
    io.emit('value', preset, property, value);
  }

  export function emitValuesChanges(preset: string, changes: { [key: string]: PropertyValue }) {
    io.emit('values', preset, changes);
  }

  export function emitPassphraseChanged(wrongPassphrase: string) {
    io.emit('passphrase', wrongPassphrase);
  }

  export function onPing(time: number) {
    io.emit('pong', time, new Date().getTime());
  }
}