import path from 'path';


class Program {
  readonly name: string = 'Conductor';
  readonly dev: boolean;
  readonly monitor: boolean;
  readonly port: number;
  readonly hostname: string;
  readonly ueHttpPort: number;
  readonly ueWebSocketPort: number;
  readonly rootFolder: string;
  readonly logger: boolean;

  constructor() {
    this.dev = false;
    this.monitor = false;
    this.port = 7000;
    this.hostname = '0.0.0.0';
    this.ueHttpPort = 30010;
    this.ueWebSocketPort = 30020;
    this.logger = false;

    for (let i = 2; i < process.argv.length; i++) {
      switch (process.argv[i]) {
        case '--dev':
          this.dev = true;
          this.port = 7001;
          break;

        case '--monitor':
          this.monitor = true;
          break;

        case '--port':
          this.port = parseInt(process.argv[i + 1]) || 7000;
          i++;
          break;

        case '--uehttp':
          this.ueHttpPort = parseInt(process.argv[i + 1]) || 30010;
          i++;
          break;

        case '--uews':
          this.ueWebSocketPort = parseInt(process.argv[i + 1]) || 30020;
          i++;
          break;

        case '--log':
          this.logger = true;
          break;

        case '--hostname':
          this.hostname = process.argv[i + 1] || '0.0.0.0';
      }
    }

    process.env.NODE_ENV = this.dev ? 'development' : 'production';
    this.rootFolder = path.resolve('./');
  }
}

const Instance = new Program();
export { Instance as Program };