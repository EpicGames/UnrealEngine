import { IPayload, IPayloads, IPreset, IPresets, IPanel, IView, ICustomStackWidget, ICustomStackTabs, PropertyValue, 
          IAsset, WidgetTypes, PropertyType, IColorPickerList, ICustomStackItem, IGroup, IExposedProperty, TabLayout } from '../../Client/src/shared';
import _ from 'lodash';
import WebSocket from 'ws';
import { Notify, Program, LogServer } from './';
import fs from 'fs-extra';
import crypto from 'crypto';


namespace UnrealApi {
  export enum PresetEvent {
    FieldsRenamed         = 'PresetFieldsRenamed',
    FieldsChanged         = 'PresetFieldsChanged',
    FieldsAdded           = 'PresetFieldsAdded',
    FieldsRemoved         = 'PresetFieldsRemoved',
    MetadataModified      = 'PresetMetadataModified',
    ActorModified         = 'PresetActorModified',
    EntitiesModified      = 'PresetEntitiesModified',
    LayoutModified        = 'PresetLayoutModified',

    ControllersAdded      = 'PresetControllersAdded',
    ControllersRemoved    = 'PresetControllersRemoved',
    ControllersModified   = 'PresetControllersModified',
    ControllersRenamed    = 'PresetControllersRenamed',
  }

  export type Presets = { 
    Presets: Partial<IPreset>[];
  };

  export type Preset = {
    Preset: IPreset;
  };

  export type View = {
    Value: string;
  };

  export type PassphraseCheck = {
    keyCorrect: boolean;
  };

  export type Request = {
    RequestId: number;
    URL: string;
    Verb: 'GET' | 'PUT' | 'POST' | 'DELETE';
  };

  export type Response = {
    RequestId: number;
    ResponseCode: number;
    ResponseBody: any;
  };

  export type BatchResponses = {
    Responses: Response[];
  };

  export type GetPropertyValue = {
    PropertyValue: PropertyValue;
  };

  export type PropertyValueSet = {
    ObjectPath: string;
    PropertyValue: PropertyValue;
  };

  export type PropertyValues = {
    PropertyValues: PropertyValueSet[];
  };

  export type Assets = {
    Assets: IAsset[];
  };

  export type RenameLabel = {
    AssignedLabel: string;
  };
}

export namespace UnrealEngine {
  let connection: WebSocket;
  let quitTimout: NodeJS.Timeout;
  let isPullingPresets: boolean;
  let isLoadingPresets: boolean;
  let autoPullTimeout: NodeJS.Timeout;
  let isOpen: boolean;

  let pendings: { [id: string]: (reply: any) => void } = {};
  let httpRequest: number = 1;
  let wsRequest: number = 1;

  let presets: IPresets = {};
  let registered: { [id: string]: boolean };
  let payloads: IPayloads = {};
  let views: { [preset: string]: IView } = {};
  let workingPassphrases: string[] = [];
  let favorites: { [preset: string]: boolean } = {};

  export async function initialize() {
    try {
      if (await fs.pathExists('./favorites.json'))
        favorites = await fs.readJSON('./favorites.json');
    } catch {
    }

    connect();
    startQuitTimeout(10 * 60 * 1000);
  }

  export function isConnected(): boolean {
    return (connection?.readyState === WebSocket.OPEN);
  }

  export function isOpenConnection(): boolean {
    return isOpen;
  }

  export function isLoading(): boolean {
    return isLoadingPresets;
  }

  function setLoading(loading: boolean) {
    isLoadingPresets = loading;
    Notify.emit('loading', loading);
  }

  function connect() {
    if (connection?.readyState === WebSocket.OPEN || connection?.readyState === WebSocket.CONNECTING)
      return;

    const address = `ws://127.0.0.1:${Program.ueWebSocketPort}`;

    connection?.removeAllListeners();

    connection = new WebSocket(address);
    connection
      .on('open', onConnected)
      .on('message', onMessage)
      .on('error', onError)
      .on('close', onClose);
  }

  export async function checkPassphrase(passphrase: string) : Promise<boolean> {
    try {
      const res = await get<UnrealApi.PassphraseCheck>(`/remote/passphrase/`, undefined, passphrase);
      if (!res?.keyCorrect)
        return false;

    } catch (error) {
      return false;
    }

    if (!workingPassphrases.includes(passphrase)) {
      workingPassphrases.push(passphrase);
      
      if (workingPassphrases.length === 1 || isOpen) {
        await pullPresets();
        pullTimer();   
      }
    }
    else {
      const index = workingPassphrases.findIndex((value:string):boolean => value === passphrase);
      workingPassphrases[index] = passphrase;
    }

    return true;
  }

  async function onConnected() {
    if (connection.readyState !== WebSocket.OPEN)
      return;

    presets = {};
    registered = {};
    payloads = {};
    views = {};
    favorites = {};
    isOpen = undefined;
    
    clearQuitTimeout();
    
    Notify.emit('connected', true);
    
    isOpen = await checkPassphrase('');

    Notify.emit('opened', isOpen);
    if (isOpen || workingPassphrases.length > 0) {
      await refresh();
      pullTimer();
    }
  }

  async function refresh() {
    try {
      await pullPresets();
    } catch (error) {
    }
  }

  async function pullTimer() {
    try {
      const { Presets } = await get<UnrealApi.Presets>('/remote/presets', undefined);

      // First check if there are any new presets
      for (const preset of Presets) {
        if (preset && !presets[preset.ID])
          await refreshPreset(preset.ID, preset.Name);
      }

      // Check for deleted presets
      const deleted = _.difference(Object.keys(presets), Presets.map(p => p.ID));
      if (deleted.length > 0) {
        for (const id of deleted) {
          delete presets[id];
          delete payloads[id];
        }

        Notify.emit('presets', getSortedPresets());
      }
    } catch (error) {
    }

    // Check for new presets once every 5 seconds
    if (isConnected())
       autoPullTimeout = setTimeout(pullTimer, 5 * 1000);
  }
 
  async function onMessage(data: WebSocket.Data) {
    const json = data.toString();

    try {
      const message = JSON.parse(json);
      if (message.RequestId) {
        const promise = pendings[message.RequestId];
        if (!promise)
          return;

        if (Program.logger)
          LogServer.logLoopback({ RequestId: message.RequestId, Stage: 'WebApp Done' }, true);
        
        if (message.Verb === "401" && workingPassphrases.includes(message.Passphrase)) {
          Notify.emitPassphraseChanged(message.Passphrase);
          workingPassphrases = workingPassphrases.filter(value => value !== message.Passphrase);

          delete pendings[message.RequestId];
          promise?.(message.ResponseCode);
          return;
        }

        delete pendings[message.RequestId];
        promise?.(message.ResponseBody);
        return;
      }

      if (message.keyCorrect !== undefined && isOpen !== message.keyCorrect) {
        isOpen = message.keyCorrect;
        Notify.emit('opened', isOpen);
        await pullPresets();
        pullTimer();
      }

      switch (message.Type) {
        case UnrealApi.PresetEvent.FieldsChanged: {
          const preset = presets[message.PresetId];
          if (!preset)
            break;

          const changes: any = {};
          for (const field of message.ChangedFields) {
            const property = preset.Exposed[field.Id];
            if (!property)
              continue;

            setPayloadValueInternal(payloads, [message.PresetId, property.ID], field.PropertyValue);
            changes[property.ID] = field.PropertyValue;
          }

          Notify.emitValuesChanges(message.PresetId, changes);
          break;
        }

        case UnrealApi.PresetEvent.FieldsAdded: {
          const preset = await populatePreset(message.Description as IPreset);
          const values = await pullPresetValues(preset);
          for (const property in values) {
            setPayloadValueInternal(payloads, [preset.ID, property], values[property]);
            Notify.emitValueChange(preset.ID, property, values[property]);
          }

          await refreshPreset(preset.ID, preset.Name);
          break;
        }

        case UnrealApi.PresetEvent.FieldsRemoved:
        case UnrealApi.PresetEvent.FieldsRenamed:
        case UnrealApi.PresetEvent.EntitiesModified: {
          await refreshPreset(message.PresetId, message.PresetName);
          break;
        }

        case UnrealApi.PresetEvent.MetadataModified: {
          await refreshPreset(message.PresetId, message.PresetName);
          refreshView(message.PresetId, message.Metadata.view, undefined);
          break;
        }

        case UnrealApi.PresetEvent.LayoutModified: {
          const preset = await populatePreset(message.Preset as IPreset);
          if (preset?.ID) {
            presets[preset.ID] = preset;
            Notify.emit('presets', getSortedPresets());
          }

          break;
        }

        case UnrealApi.PresetEvent.ControllersAdded:
        case UnrealApi.PresetEvent.ControllersRemoved:
        case UnrealApi.PresetEvent.ControllersRenamed: {
          await refreshPreset(message.PresetId, message.PresetName);
          break;
        }

        case UnrealApi.PresetEvent.ControllersModified: {
          const preset = presets[message.PresetId];
          if (!preset)
            break;

          const changes: any = {};
          for (let i = 0; i < message.ModifiedControllers.Controllers.length; i++) {
            const id = message.ModifiedControllers.Controllers[i].ID;
            const controller = preset.Exposed[id];
            if (!controller)
              continue;

            const value = message.ModifiedControllers.ChangedValues[i];
            setPayloadValueInternal(payloads, [message.PresetId, id], value);
            changes[id] = value;
          }

          Notify.emitValuesChanges(message.PresetId, changes);          
          break;
        }
      }

    } catch (error) {
      console.log('Failed to process message', error?.message, json);
    }
  }
 
  function onError(error: Error) {
    console.log('UE Remote WebSocket:', error.message);
  }

  function onClose() {
    presets = {};
    registered = {};
    payloads = {};
    views = {};
    favorites = {};
    isOpen = undefined;

    Notify.emit('connected', false);

    startQuitTimeout();
    clearTimeout(autoPullTimeout);
    autoPullTimeout = null;

    setTimeout(connect, 1000);
  }

  function quit() {
    process.exit(1);
  }

  function startQuitTimeout(timeout: number = 15 * 1000) {
    if (Program.monitor && !quitTimout)
      quitTimout = setTimeout(quit, timeout);
  }

  function clearQuitTimeout() {
    if (quitTimout)
      clearTimeout(quitTimout);

    quitTimout = undefined;
  }

  function verifyConnection() {
    if (connection?.readyState !== WebSocket.OPEN)
      throw new Error('Websocket is not connected');
  }

  function send(message: string, parameters: any, passphrase?: string, ip?: string) {
    verifyConnection();
    const Id = parameters?.RequestId ?? wsRequest++;
    passphrase = passphrase ?? workingPassphrases[0];
	  ip = ip ?? "";
    connection.send(JSON.stringify({ MessageName: message, Id, Passphrase: passphrase, ForwardedFor: ip, Parameters: parameters}));
  }

  function http<T>(Verb: string, URL: string, Body?: object, wantAnswer?: boolean, passphrase?: string, ip?: string): Promise<T> {
    const RequestId = httpRequest++;
    const payload = { RequestId, Verb, URL, Body };
    if (Program.logger)
      LogServer.logLoopback(payload);

    send('http', payload, passphrase, ip);
    if (!wantAnswer)
      return Promise.resolve(null);

    return new Promise(resolve => {
      pendings[RequestId] = resolve;
    });
  }

  function get<T>(url: string, ip: string, passphrase?: string): Promise<T> {
    return http<T>('GET', url, undefined, true, passphrase, ip);
  }

  function put<T>(url: string, body: object, ip: string, passphrase?: string): Promise<T> {
    return http<T>('PUT', url, body, true, passphrase, ip);
  }

  function registerToPreset(PresetName: string): void {
    send('preset.register', { PresetName, IgnoreRemoteChanges: true });
  }

  function unregisterPreset(PresetName: string) {
    send('preset.unregister', { PresetName });
  }

  function getSortedPresets() {
    return _.orderBy(Object.values(presets), ['IsFavorite', 'Name'], ['desc', 'asc']);
  }

  export async function getPresets(): Promise<IPreset[]> {
    return getSortedPresets();
  }

  async function pullPresets(): Promise<void> {
    if (isPullingPresets)
      return;

    isPullingPresets = true;
    setLoading(true);

    try { 
      const { Presets } = await get<UnrealApi.Presets>('/remote/presets', undefined);

      for (const preset of Presets ?? []) {
        const stub = preset as IPreset;
        stub.Exposed = {};
        stub.ExposedFunctions = [];
        stub.ExposedProperties = [];
        stub.Groups = [];
        stub.IsFavorite = !!favorites?.[preset.ID];
        presets[preset.ID] = stub;
      }

      Notify.emit('presets', getSortedPresets());

    } catch (error) {
      console.log('Failed to pull presets data');
    }

    isPullingPresets = false;
    setLoading(false);
  }

  async function pullPreset(id: string, name?: string, ip?: string): Promise<IPreset> {
    try {
      if (!presets[id]) {
        const res = await get<UnrealApi.View>(`/remote/preset/${id}/metadata/view`, ip);
        refreshView(id, res?.Value, ip);
      }

      const { Preset } = await get<UnrealApi.Preset>(`/remote/preset/${id}`, ip);
      if (!Preset)
        return null;

      await populatePreset(Preset);
      return Preset;
    } catch (error) {
      console.log(`Failed to pull preset '${name || id}' data`);
    }
  }

  async function refreshPreset(id: string, name?: string, ip?: string): Promise<IPreset> {
    const preset = await pullPreset(id, name, ip);
    if (!preset)
      return;

    presets[id] = preset;
    Notify.emit('presets', getSortedPresets());
    return preset;
  }

  async function populatePreset(preset: IPreset) {
    preset.ExposedProperties = [];
    preset.ExposedFunctions = [];
    preset.Exposed = {};

    for (const Group of preset.Groups) {
      for (const Property of Group.ExposedProperties) {
        Property.Type = Property.UnderlyingProperty.Type;
        Property.TypePath = Property.UnderlyingProperty.TypePath;
        preset.Exposed[Property.ID] = Property;
      }

      for (const Function of Group.ExposedFunctions) {
        if (!Function.Metadata)
          Function.Metadata = {};

        Function.Type = PropertyType.Function;
        Function.Metadata.Widget = WidgetTypes.Button;
        preset.Exposed[Function.ID] = Function;
      }

      preset.ExposedProperties.push(...Group.ExposedProperties);
      preset.ExposedFunctions.push(...Group.ExposedFunctions);
    }

    if (preset.Controllers) {
      const Controllers: IGroup = { Name: 'Controllers', ExposedProperties: [], ExposedFunctions: [] };
      preset.Groups.push(Controllers);
      for (const Controller of preset.Controllers) {
        if (!Controller.Metadata)
          Controller.Metadata = {};

        Controller.UnderlyingProperty = {
          Name: Controller.Name,
          Metadata: Controller.Metadata,
          Description: '',
          Type: Controller.Type,
          TypePath: Controller.TypePath
        };

        preset.Exposed[Controller.ID] = Controller;
        Controllers.ExposedProperties.push(Controller);
      }
    }

    preset.IsFavorite = favorites?.[preset.ID];
    return preset;
  }

  function refreshView(id: string, viewJson: string, ip: string) {
    try {
      if (!viewJson) {
        views[id] = {
          tabs: [{ name: 'Tab 1', layout: TabLayout.Empty, icon: '' }]
        };

        Notify.onViewChange(id, views[id], true, ip);
        return;
      }

      const view = JSON.parse(viewJson) as IView;
      if (equal(view, views[id]))
        return;

      for (const tab of view.tabs) {
        if (!tab.panels)
          continue;

        for (const panel of tab.panels)
          setPanelIds(panel);
      }

      views[id] = view;
      Notify.onViewChange(id, view, true, ip);
    } catch (error) {
      console.log('Failed to parse View of Preset', id);
    }
  }

  function setPanelIds(panel: IPanel) {
    if (!panel.id)
      panel.id = crypto.randomBytes(16).toString('hex');

    if (panel.widgets)
      return setWidgetsId(panel.widgets);

    if (panel.items)
      for (const item of panel.items) {
        if (!item.id)
          item.id = crypto.randomBytes(16).toString('hex');

        for (const panel of item.panels)
          setPanelIds(panel);
      }
  }

  function setWidgetsId(widgets?: ICustomStackWidget[]) {
    if (!widgets)
      return;

    for (const widget of widgets) {
      if (!widget.id)
        widget.id = crypto.randomBytes(16).toString('hex');

      switch (widget.widget) {
        case WidgetTypes.Tabs: {
          const children = (widget as ICustomStackTabs).tabs ?? [];
          for (const child of children) {
            if (!child.id)
              child.id = crypto.randomBytes(16).toString('hex');
  
            setWidgetsId(child.widgets);
          }          
          break;
        }

        case WidgetTypes.ColorPickerList: {
          const children = (widget as IColorPickerList).items ?? [];
          for(const child of children) {
            if (!child.id)
              child.id = crypto.randomBytes(16).toString('hex');   
          }
          break;
        }
      }
   }
  }

  function equal(a: any, b: any): boolean {
    if (a === b)
      return true;
    
    if (!!a !== !!b || typeof(a) !== typeof(b))
      return false;

    if (Array.isArray(a) && Array.isArray(b)) {
      if (a.length !== b.length)
        return false;

      for (let i = 0; i < a.length; i++) {
        if (!equal(a[i], b[i]))
          return false;
      }

      return true;
    }

    if (typeof(a) === 'object' && typeof(b) === 'object') {
      if (!equal(Object.keys(a), Object.keys(b)))
        return false;

      for (const key in a) {
        if (!equal(a[key], b[key]))
          return false;
      }

      return true;
    }

    return false;
  }

  function getPropertyValueRequest(Preset: IPreset, Property: IExposedProperty, IsController: boolean) {
    return ({
      RequestId: httpRequest++,
      Verb: 'GET',
      URL: `/remote/preset/${Preset.ID}/property/${Property.ID}`,
      PropertyId: Property.ID,
      IsController,
    });
  }

  async function pullPresetValues(Preset: IPreset, ip?: string): Promise<IPayload> {
    const Requests = [];
    for (const Property of Preset.ExposedProperties)
      Requests.push(getPropertyValueRequest(Preset, Property, false));

    if (Preset.Controllers) {
      for (const Controller of Preset.Controllers)
        Requests.push(getPropertyValueRequest(Preset, Controller, true));
    }

    const updatedPayloads: IPayloads = {};
    const answers = await put<UnrealApi.BatchResponses>('/remote/batch', { Requests }, ip);
    for (let i = 0; i < Requests.length; i++) {
      const { PropertyId, IsController } = Requests[i];
      try {
        let value;
        if (IsController) {
          const body = answers.Responses[i].ResponseBody as UnrealApi.GetPropertyValue;
          value = body?.PropertyValue;
        } else {
          const body = answers.Responses[i].ResponseBody as UnrealApi.PropertyValues;
          value = body?.PropertyValues?.[0]?.PropertyValue;
        }

        setPayloadValueInternal(updatedPayloads, [Preset.ID, PropertyId], value);
      } catch (error) {
        console.log(`Failed to get value of Preset: ${Preset.Name}, Property: ${PropertyId}`);
      }
    }

    return updatedPayloads[Preset.ID];
  }

  export async function loadPreset(id: string, ip?: string): Promise<IPreset> {
    if (registered[id] && presets[id])
      return presets[id];

    registered[id] = true;
    registerToPreset(id);
    const preset = await refreshPreset(id, undefined, ip);
    if (!preset)
      return;

    payloads[id] = await pullPresetValues(preset, ip);
    Notify.emit('payloads', payloads);
    
    const res = await get<UnrealApi.View>(`/remote/preset/${id}/metadata/view`, ip);
    refreshView(id, res?.Value, ip);
  }

  export async function getPayload(preset: string): Promise<IPayload> {
    return payloads[preset];
  }

  export async function getPayloads(): Promise<IPayloads> {
    return { ...payloads };
  }

  export async function setPayload(preset: string, payload: IPayload): Promise<void> {
    payloads[preset] = payload;
  }

  export async function setPayloadValue(preset: string, property: string, value: PropertyValue, ip: string): Promise<void> {
    try {
      const body: any = { GenerateTransaction: true };
      if (value !== null) {
        setPayloadValueInternal(payloads, [preset, property], value);
        Notify.emitValueChange(preset, property, value);
        body.PropertyValue = value;
      } else {
        body.ResetToDefault = true; 
      }

      if (Program.logger) {
        const presetObj = presets[preset];
        const log = {
          RequestId: httpRequest,
          Stage: 'Set Property',
          Preset: presetObj?.Name,
          Property: presetObj?.Exposed[property]?.DisplayName,
          Value: value
        };

        LogServer.logLoopback(log);
      }

      await put(`/remote/preset/${preset}/property/${property}`, body, ip);

      if (value === null) {
        const ret = await get<UnrealApi.PropertyValues>(`/remote/preset/${preset}/property/${property}`, ip);
        value = ret.PropertyValues?.[0]?.PropertyValue;
        if (value !== undefined) {
          setPayloadValueInternal(payloads, [preset, property], value);
          Notify.emitValueChange(preset, property, value);
        }
      }
    } catch (err) {
      console.log('Failed to set preset data:', err.message);
    }
  }

  function setPayloadValueInternal(data: IPayloads, path: string[], value: PropertyValue): void {
    if (!data || !path.length)
      return;

    let element: any = data;
    for (let i = 0; i < path.length - 1; i++) {
      const property = path[i];
      if (!element[property])
        element[property] = {};
      
      element = element[property];
    }

    element[ _.last(path) ] = value;
  }

  export async function executeFunction(preset: string, func: string, args: Record<string, any>, ip: string): Promise<void> {
    try {
      const url = `/remote/preset/${preset}/function/${func}`;
      await put(url, { Parameters: args, GenerateTransaction: true }, ip);
    } catch (err) {
      console.log('Failed to set execute function call:', err.message);
    }
  }

  export async function setPresetPropertyMetadata(preset: string, property: string, metadata: string, value: string, ip: string) {
    try {

      const url = `/remote/preset/${preset}/property/${property}/metadata/${metadata}`;
      await put(url, { value }, ip);
      const prop = presets[preset]?.Exposed[property];
      if (prop) {
        prop.Metadata[metadata] = value;
        Notify.emit('presets', getSortedPresets());
      }
    } catch (err) {
      console.log('Failed to set property metadata');
    }
  }

  export async function rebindProperties(preset: string, properties: string[], target: string, ip: string) {
    const body = {
      objectPath: '/Script/RemoteControlWebInterface.Default__RCWebInterfaceBlueprintLibrary',
      functionName: 'RebindProperties',
      parameters: {
        PresetId: preset,
        PropertyIds: properties,
        NewOwner: target,
      }
    };

    await put('/remote/object/call', body, ip);

    // We need to fetch values of new owner
    const changes: any = {};
    for (const property of properties) {
      const ret = await get<UnrealApi.PropertyValues>(`/remote/preset/${preset}/property/${property}`, ip);
      const value = ret.PropertyValues?.[0]?.PropertyValue;
      if (value !== undefined) {
        setPayloadValueInternal(payloads, [preset, property], value);
        changes[property] = value;
      }
    }

    Notify.emitValuesChanges(preset, changes);
  }

  export async function favoritePreset(preset: string, value: boolean): Promise<void> {
    if (!presets[preset])
      return;

    presets[preset].IsFavorite = value;

    if (value)
      favorites[preset] = true;
    else
      delete favorites[preset];

    await fs.writeJson('./favorites.json', favorites, { spaces: 2 });
    Notify.emit('presets', getSortedPresets());
  }

  export async function getView(preset: string): Promise<IView> {
    return views[preset];
  }

  export async function setView(preset: string, view: IView, ip: string): Promise<void> {
    for (const tab of view.tabs) {
      if (!tab.panels)
        continue;

      for (const panel of tab.panels)
        setPanelIds(panel);
    }

    views[preset] = view;
    const Value = JSON.stringify(view);
    await put(`/remote/preset/${preset}/metadata/view`, { Value }, ip);
  }

  export async function search(query: string, types: string[], prefix: string, filterArgs: any, count: number, callback: (assets: IAsset[]) => void, ip: string): Promise<void> {
    const ret = await put<UnrealApi.Assets>('/remote/search/assets', {
      Query: query,
      Limit: count,
      
      Filter: {
        ClassNames: types,
        PackagePaths: [prefix],
        RecursivePaths: true,
        RecursiveClasses: true,
        ...filterArgs,
      }
    }, ip);

    callback?.(ret.Assets);
  }

  export function proxy(method: 'GET' | 'PUT', url: string, body: any, ip: string): Promise<any> {
    if (!method || !url)
      return Promise.resolve({});

    switch (method) {
      case 'GET':
        return get(url, ip);

      case 'PUT':
        if (body)
          return put(url, body, ip);
        break;
    }

    return Promise.resolve({});
  }

  export function thumbnail(asset: string, ip: string): Promise<any> {
    return put<string>('/remote/object/thumbnail', { ObjectPath: asset }, ip)
            .then(base64 => Buffer.from(base64, 'base64'))
            .catch(error => null);
  }
}