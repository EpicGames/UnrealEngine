{
  "compilerOptions": {
    "module": "commonjs",
    "resolveJsonModule": true,
    "esModuleInterop": true,
    "target": "es2018",
    "noImplicitAny": true,
    "moduleResolution": "node",
    "sourceMap": false,
    "outDir": "build",
    "baseUrl": ".",
    "experimentalDecorators": true,
    "emitDecoratorMetadata": true,
    "plugins": [
      { "name": "typescript-tslint-plugin" }
    ],
    "paths": {
      "@Shared": ["../Client/src/shared/index.ts"]
    },
  },
  "include": ["index.ts", "../Client/src/shared"]
}