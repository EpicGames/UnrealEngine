{
    "defaultSeverity": "error",
    "extends": [
    ],
    "jsRules": {},
    "rules": {
        "trailing-comma": [ false ],
        "no-console": [ true, "warn" ],
        "semicolon": [ true, "always" ]
    },
    "rulesDirectory": []
}