@echo off
setlocal enabledelayedexpansion
setlocal enableextensions

pushd "%~dp0"

SET NodeVersion=v16.17.0
SET DownloadedNodeFolder=Node-%NodeVersion%
SET NodeName=node-%NodeVersion%-win-x64

if not exist "%DownloadedNodeFolder%\" (
  echo Downloading NodeJS for Windows...
  
  @REM Download nodejs and follow redirects.
  curl -s -L -o .\%NodeName%.zip "https://nodejs.org/dist/%NodeVersion%/%NodeName%.zip"

  @REM Only if download succssed
  if exist %NodeName%.zip (
    @REM cleanup node_modules, as they might be of an different nodejs bersion
    CALL Clean.bat

    @REM Unarchive the .zip
    tar -xf %NodeName%.zip

    @REM Rename the extracted, versioned, directory that contains the NodeJS binaries to simply "Node".
    ren "%NodeName%\" "%DownloadedNodeFolder%"

    @REM Delete the downloaded node.zip
    del %NodeName%.zip
  ) else (
    echo Failed to download NodeJS %NodeVersion%
  )
)

if exist "%DownloadedNodeFolder%\" (
  @REM Add downloaded nodejs version to be first in line
  echo Using downloaded NodeJS version from folder %cd%\%DownloadedNodeFolder%
  SET "PATH=%cd%\%DownloadedNodeFolder%;%PATH%;"
)

@REM Add default nodejs installation folder to path, in case it was not added / overwritten
SET PATH=%PATH%;%ProgramFiles%\nodejs\


@REM Check if nodejs is in the env variable PATH 
for %%X in (node.exe) do (set node=%%~$PATH:X)
if not defined node (
  echo ERROR: Couldn't find node.js installed, Please install latest nodejs from https://nodejs.org/en/download/
  exit 1
)


@REM Let's check if it is a modern nodejs
node -e "process.exit( process.versions.node.split('.')[0] );"
echo Using Node.js version %errorlevel%

if %errorlevel% LSS 14 (
  echo ERROR: installed node.js version is too old, please install latest nodejs from https://nodejs.org/en/download/
  exit 1
)

if %errorlevel% GEQ 17 (
  @REM Due to changes on Node.js v17, --openssl-legacy-provider was added for handling key size on OpenSSL v3
  SET NODE_OPTIONS=--openssl-legacy-provider
)

@REM redirecting all command line arguments to node script
node Scripts/start.js %*
