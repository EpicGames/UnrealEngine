#!/bin/sh

set -e

NodeOS=linux
if [[ $OSTYPE == 'darwin'* ]]; then
  NodeOS=darwin
fi

NodeVersion=v16.17.0
DownloadedNodeFolder=node-${NodeVersion}-${NodeOS}-x64
NodeName=${DownloadedNodeFolder}.tar.gz
NodeFolderBasePath=$(cd "$(dirname "$0")"; pwd -P)

if [ ! -d "${DownloadedNodeFolder}" ]; then
  echo Downloading NodeJS ...
  
  # Download nodejs and follow redirects.
  curl -s -L -o ./${NodeName} "https://nodejs.org/dist/${NodeVersion}/${NodeName}"

  # Only if download succssed
  if [ -f "${NodeName}" ]; then
    # Unarchive the archive
	tar -xf ${NodeName}
    

    # Delete the downloaded node.zip
	rm ${NodeName}
	
  else
    echo Failed to download NodeJS ${NodeVersion}
  fi
fi


if [ -d "${NodeFolderBasePath}/${DownloadedNodeFolder}" ]; then
  # Add downloaded nodejs version to be first in line
  echo Using downloaded NodeJS version
  export PATH="${NodeFolderBasePath}/${DownloadedNodeFolder}/bin:$PATH"
fi



# First we check if nodejs is installed
if ! command -v node > /dev/null ; then
  echo "ERROR: Couldn't find node.js installed..., Please install latest nodejs from https://nodejs.org/en/download/"
  exit 1
fi

# Check if npm is installed
if ! command -v npm > /dev/null ; then
  echo "ERROR: Couldn't find npm installed..., Please install npm"
  exit 1
fi

# Let's check if it is a modern nodejs
VERSION=$(node -e "console.log( process.versions.node.split('.')[0] );")
echo "Found Node.js version ${VERSION}"

if [ ${VERSION} -lt 14 ] ; then
  echo "ERROR: installed node.js version is too old (${VERSION}) :\( Please install latest nodejs from https://nodejs.org/en/download/"
  exit 1
fi

if [ ${VERSION} -ge 17 ] ; then
  #Due to changes on Node.js v17, --openssl-legacy-provider was added for handling key size on OpenSSL v3
  export NODE_OPTIONS=--openssl-legacy-provider
fi


FOLDER=$(dirname "$0")
node "${FOLDER}/Scripts/start.js" "$@"
