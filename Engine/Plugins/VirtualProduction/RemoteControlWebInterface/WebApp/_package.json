{
  "name": "conductor",
  "version": "0.3.93",
  "scripts": {
    "postinstall": "concurrently \"cd ./Client && npm install && cd ..\" \"cd ./Server && npm install && cd ..\"",
    "build": "node ./Scripts/build.js",
    "dev": "concurrently \"cd ./Client && npm start && cd ..\" \"cd Server && npm start && cd ..\"",
    "start": "pushd Server && npm run serve && popd"
  },
  "dependencies": {},
  "devDependencies": {
    "concurrently": "^5.3.0",
    "fs-extra": "^9.0.1"
  }
}
